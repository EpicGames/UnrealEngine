{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.4",
	"FriendlyName": "MassCrowd",
	"Description": "Spline based AI crowd system",
	"Category": "AI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MassCrowd",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ZoneGraph",
			"Enabled": true
		},
		{
			"Name": "MassGameplay",
			"Enabled": true
		},
		{
			"Name": "MassAI",
			"Enabled": true
		},
		{
			"Name": "StateTree",
			"Enabled": true
		}
	]
}