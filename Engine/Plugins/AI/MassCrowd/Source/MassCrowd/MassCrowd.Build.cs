// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassCrowd: ModuleRules
	{
		public MassCrowd(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AIModule",
					"MassCore",
					"MassEntity",
					"Core",
					"CoreUObject",
					"Engine",
					"MassActors",
					"MassCommon",
					"MassLOD",
					"MassMovement",
					"MassNavigation",
					"MassZoneGraphNavigation",
					"MassReplication",
					"MassAIReplication",
					"MassSignals",
					"MassSimulation",
					"MassSpawner",
					"MassRepresentation",
					"MassAIBehavior",
					"NetCore",
					"StateTreeModule",
					"ZoneGraph",
					"ZoneGraphAnnotations",
					"ZoneGraphDebug"
				}
			);

			SetupIrisSupport(Target);
		}
	}
}