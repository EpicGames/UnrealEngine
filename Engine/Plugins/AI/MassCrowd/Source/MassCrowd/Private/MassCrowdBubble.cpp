// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdBubble.h"
#include "MassEntityManager.h"
#include "Net/UnrealNetwork.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdBubble)

namespace UE::Mass::Crowd
{
	bool bDebugReplicationPositions = false;
	FAutoConsoleVariableRef CVarbDebugReplication(TEXT("mass.debug.CrowdReplicationPositions"), bDebugReplicationPositions, TEXT("Crowd debug replication positions on server and client"), ECVF_Cheat);

#if WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR
	// @todo provide a better way of selecting agents to debug
	constexpr int32 MaxAgentsDraw = 300;

	void DebugDrawReplicatedAgent(FMassEntityHandle Entity, const FMassEntityManager& EntityManager)
	{
		static const FVector DebugCylinderHeight = FVector(0.f, 0.f, 200.f);
		static constexpr float DebugCylinderRadius = 50.f;

		const FMassEntityView EntityView(EntityManager, Entity);

		const FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();
		const FMassNetworkIDFragment& NetworkIDFragment = EntityView.GetFragmentData<FMassNetworkIDFragment>();

		const FVector& Pos = TransformFragment.GetTransform().GetLocation();
		const uint32 NetworkID = NetworkIDFragment.NetID.GetValue();

		// Multiply by a largeish number that is not a multiple of 256 to separate out the color shades a bit
		const uint32 InitialColor = NetworkID * 20001;

		const uint8 NetworkIdMod3 = NetworkID % 3;
		FColor DebugCylinderColor;

		// Make a deterministic color from the Mod by 3 to vary how we create it
		if (NetworkIdMod3 == 0)
		{
			DebugCylinderColor = FColor(InitialColor % 256, InitialColor / 256 % 256, InitialColor / 256 / 256 % 256);
		}
		else if (NetworkIdMod3 == 1)
		{
			DebugCylinderColor = FColor(InitialColor / 256 / 256 % 256, InitialColor % 256, InitialColor / 256 % 256);
		}
		else
		{
			DebugCylinderColor = FColor(InitialColor / 256 % 256, InitialColor / 256 / 256 % 256, InitialColor % 256);
		}

		const UWorld* World = EntityManager.GetWorld();
		if (World != nullptr && World->GetNetMode() == NM_Client)
		{
			DrawDebugCylinder(World, Pos, Pos + 0.5f * DebugCylinderHeight, DebugCylinderRadius, /*segments = */24, DebugCylinderColor);
		}
		else
		{
			DrawDebugCylinder(World, Pos + 0.5f * DebugCylinderHeight, Pos + DebugCylinderHeight, DebugCylinderRadius, /*segments = */24, DebugCylinderColor);
		}
	}
#endif // WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR
}

#if WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR
void FMassCrowdClientBubbleHandler::DebugValidateBubbleOnServer()
{
	Super::DebugValidateBubbleOnServer();

#if UE_REPLICATION_COMPILE_SERVER_CODE
	if (UE::Mass::Crowd::bDebugReplicationPositions)
	{
		const FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

		// @todo cap at MaxAgentsDraw for now
		const int32 MaxAgentsDraw = FMath::Min(UE::Mass::Crowd::MaxAgentsDraw, (*Agents).Num());

		for (int32 Idx = 0; Idx < MaxAgentsDraw; ++Idx)
		{
			const FCrowdFastArrayItem& CrowdItem = (*Agents)[Idx];

			const FMassAgentLookupData& LookupData = AgentLookupArray[CrowdItem.GetHandle().GetIndex()];

			check(LookupData.Entity.IsSet());

			UE::Mass::Crowd::DebugDrawReplicatedAgent(LookupData.Entity, EntityManager);
		}
	}
#endif // UE_REPLICATION_COMPILE_SERVER_CODE
}
#endif // WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR

#if WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR
void FMassCrowdClientBubbleHandler::DebugValidateBubbleOnClient()
{
	Super::DebugValidateBubbleOnClient();

	if (UE::Mass::Crowd::bDebugReplicationPositions)
	{
		const FMassEntityManager& EntityManager = Serializer->GetEntityManagerChecked();

		UMassReplicationSubsystem* ReplicationSubsystem = Serializer->GetReplicationSubsystem();
		check(ReplicationSubsystem);

		// @todo cap at MaxAgentsDraw for now
		const int32 MaxAgentsDraw = FMath::Min(UE::Mass::Crowd::MaxAgentsDraw, (*Agents).Num());

		for (int32 Idx = 0; Idx < MaxAgentsDraw; ++Idx)
		{
			const FCrowdFastArrayItem& CrowdItem = (*Agents)[Idx];

			const FMassReplicationEntityInfo* EntityInfo = ReplicationSubsystem->FindMassEntityInfo(CrowdItem.Agent.GetNetID());

			check(EntityInfo->Entity.IsSet());

			UE::Mass::Crowd::DebugDrawReplicatedAgent(EntityInfo->Entity, EntityManager);
		}
	}
}
#endif // WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void FMassCrowdClientBubbleHandler::PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize)
{
	auto AddRequirementsForSpawnQuery = [this](FMassEntityQuery& InQuery)
	{
		PathHandler.AddRequirementsForSpawnQuery(InQuery);
		TransformHandler.AddRequirementsForSpawnQuery(InQuery);
	};

	auto CacheFragmentViewsForSpawnQuery = [this](FMassExecutionContext& InExecContext)
	{
		PathHandler.CacheFragmentViewsForSpawnQuery(InExecContext);
		TransformHandler.CacheFragmentViewsForSpawnQuery(InExecContext);
	};

	auto SetSpawnedEntityData = [this](const FMassEntityView& EntityView, const FReplicatedCrowdAgent& ReplicatedEntity, const int32 EntityIdx)
	{
		PathHandler.SetSpawnedEntityData(EntityView, ReplicatedEntity.GetReplicatedPathData(), EntityIdx);
		TransformHandler.SetSpawnedEntityData(EntityIdx, ReplicatedEntity.GetReplicatedPositionYawData());
	};

	auto SetModifiedEntityData = [this](const FMassEntityView& EntityView, const FReplicatedCrowdAgent& Item)
	{
		PostReplicatedChangeEntity(EntityView, Item);
	};

	PostReplicatedAddHelper(AddedIndices, AddRequirementsForSpawnQuery, CacheFragmentViewsForSpawnQuery, SetSpawnedEntityData, SetModifiedEntityData);

	PathHandler.ClearFragmentViewsForSpawnQuery();
	TransformHandler.ClearFragmentViewsForSpawnQuery();
}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void FMassCrowdClientBubbleHandler::FMassCrowdClientBubbleHandler::PostReplicatedChange(const TArrayView<int32> ChangedIndices, int32 FinalSize)
{
	auto SetModifiedEntityData = [this](const FMassEntityView& EntityView, const FReplicatedCrowdAgent& Item)
	{
		PostReplicatedChangeEntity(EntityView, Item);
	};

	PostReplicatedChangeHelper(ChangedIndices, SetModifiedEntityData);
}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
void FMassCrowdClientBubbleHandler::PostReplicatedChangeEntity(const FMassEntityView& EntityView, const FReplicatedCrowdAgent& Item) const
{
	PathHandler.SetModifiedEntityData(EntityView, Item.GetReplicatedPathData());

	// No need to call TransformHandler as that only gets replicated when an agent is added to the bubble
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

AMassCrowdClientBubbleInfo::AMassCrowdClientBubbleInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Serializers.Add(&CrowdSerializer);
}

void AMassCrowdClientBubbleInfo::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;

	// Technically, this doesn't need to be PushModel based because it's a FastArray and they ignore it.
	DOREPLIFETIME_WITH_PARAMS_FAST(AMassCrowdClientBubbleInfo, CrowdSerializer, SharedParams);
}
