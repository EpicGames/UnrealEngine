// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdLaneClosingTest.h"
#include "Engine/World.h"
#include "MassCrowdSubsystem.h"
#include "ZoneGraphRenderingUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdLaneClosingTest)

void UZoneGraphCloseCrowdLaneTest::OnLaneLocationUpdated(const FZoneGraphLaneLocation& PrevLaneLocation, const FZoneGraphLaneLocation& NextLaneLocation)
{
	if (CrowdSubsystem == nullptr)
	{
		return;
	}

	// Extra checks here, because GetCrowdLaneData() will ensure() on bad handle. We can end up here when Crowd subsystem has already been torn down.
	if (PrevLaneLocation.IsValid() && CrowdSubsystem->HasCrowdDataForZoneGraph(PrevLaneLocation.LaneHandle.DataHandle))
	{
		CrowdSubsystem->SetLaneState(PrevLaneLocation.LaneHandle, PrevLaneState);
	}

	if (NextLaneLocation.IsValid() && CrowdSubsystem->HasCrowdDataForZoneGraph(NextLaneLocation.LaneHandle.DataHandle))
	{
		PrevLaneState = CrowdSubsystem->GetLaneState(NextLaneLocation.LaneHandle);
		CrowdSubsystem->SetLaneState(NextLaneLocation.LaneHandle, LaneState);
	}

	LaneLocation = NextLaneLocation;
}

void UZoneGraphCloseCrowdLaneTest::Draw(FPrimitiveDrawInterface* PDI) const
{
	if (!LaneLocation.IsValid())
	{
		return;
	}
	
	const FZoneGraphStorage* Storage = OwnerComponent->GetZoneGraphStorage(LaneLocation.LaneHandle);
	if (Storage != nullptr)
	{
		TArray<FZoneGraphLaneHandle> Lanes;
		Lanes.Push(LaneLocation.LaneHandle);

		FColor Color = FColor::White;
		switch (LaneState)
		{
		case ECrowdLaneState::Opened:		Color = FColor::Emerald;	break;
		case ECrowdLaneState::Closed:		Color = FColor::Red;		break;
		default:
			ensureMsgf(false, TEXT("Unhandled lane state %d"), (int32)LaneState);
			break;
		}

		static constexpr float LaneLineThickness = 25.0f;
		UE::ZoneGraph::RenderingUtilities::DrawLanes(*Storage, PDI, Lanes, Color, LaneLineThickness);
	}	
}

void UZoneGraphCloseCrowdLaneTest::OnOwnerSet()
{
	if (OwnerComponent == nullptr)
	{
		CrowdSubsystem = nullptr;
		return;
	}

	CrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(OwnerComponent->GetWorld());
}
