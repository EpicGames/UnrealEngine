// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdLaneDataRenderingComponent.h"

#include "Engine/Engine.h"
#include "MassCrowdSettings.h"
#include "MassCrowdSubsystem.h"
#include "PrimitiveViewRelevance.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphRenderingComponent.h"
#include "ZoneGraphSubsystem.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdLaneDataRenderingComponent)

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
void UMassCrowdLaneDataRenderingComponent::OnRegister()
{
	Super::OnRegister();

	if (GetWorld()->IsGameWorld())
	{
		DebugTextDrawingDelegateHandle = UDebugDrawService::Register(TEXT("Navigation"), FDebugDrawDelegate::CreateUObject(this, &UMassCrowdLaneDataRenderingComponent::DebugDrawOnCanvas));

		// Track lane state changes
		if (UMassCrowdSubsystem* MassCrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(GetWorld()))
		{
			OnLaneStateChangedDelegateHandle = MassCrowdSubsystem->DebugOnMassCrowdLaneStateChanged.AddLambda([this]()
			{
				MarkRenderStateDirty();
			});
		}
	}

#if WITH_EDITOR
	// Track render settings changes
	OnLaneRenderSettingsChangedDelegateHandle = GetDefault<UMassCrowdSettings>()->OnMassCrowdLaneRenderSettingsChanged.AddLambda([this]()
	{
		MarkRenderStateDirty();
	});
#endif
}

void UMassCrowdLaneDataRenderingComponent::OnUnregister()
{
	UDebugDrawService::Unregister(DebugTextDrawingDelegateHandle);

	if (UMassCrowdSubsystem* MassCrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(GetWorld()))
	{
		MassCrowdSubsystem->DebugOnMassCrowdLaneStateChanged.Remove(OnLaneStateChangedDelegateHandle);
	}

#if WITH_EDITOR
	GetDefault<UMassCrowdSettings>()->OnMassCrowdLaneDataSettingsChanged.Remove(OnLaneRenderSettingsChangedDelegateHandle);
#endif

	Super::OnUnregister();
}

FPrimitiveSceneProxy* UMassCrowdLaneDataRenderingComponent::CreateSceneProxy()
{
	class FMassCrowdLaneDataSceneProxy final : public FZoneGraphSceneProxy
	{
	public:
		friend class UCrowdLaneDataRenderComponent;

		virtual SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FMassCrowdLaneDataSceneProxy(const UMassCrowdLaneDataRenderingComponent& InComponent, const AZoneGraphData& ZoneGraph)
			: FZoneGraphSceneProxy(InComponent, ZoneGraph)
			, Component(&InComponent)
		{
			const UZoneGraphSubsystem* ZoneGraphSubsystem = UWorld::GetSubsystem<UZoneGraphSubsystem>(Component->GetWorld());
			const UMassCrowdSubsystem* MassCrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(Component->GetWorld());
			const UMassCrowdSettings* Settings = GetDefault<UMassCrowdSettings>();
			checkf(Settings, TEXT("Settings default object is always expected to be valid"));

			// The subsystems could be removed during world tear down
			if (ZoneGraphSubsystem == nullptr || MassCrowdSubsystem == nullptr)
			{
				return;
			}

			const FZoneGraphStorage& Storage = ZoneGraph.GetStorage();
			// Make sure we still have lane data since we can end up here when Crowd subsystem has already been torn down.
			if (!MassCrowdSubsystem->HasCrowdDataForZoneGraph(Storage.DataHandle))
			{
				return;
			}

			// Nothing to render if none of these options are enabled
			if (!(Settings->bDisplayStates || Settings->bDisplayDensities))
			{
				return;
			}

			const FZoneGraphTag CrowdTag = Settings->CrowdTag;
			const float LaneThickness = Settings->LaneBaseLineThickness;
			const float IntersectionLaneThickness = Settings->IntersectionLaneScaleFactor * LaneThickness;
			const FVector Offset(0.f, 0.f, Settings->LaneRenderZOffset);

			const float LaneDensityThickness = Settings->LaneDensityScaleFactor * LaneThickness;
			const float IntersectionLaneDensityThickness = Settings->LaneDensityScaleFactor * IntersectionLaneThickness;
			const FVector LaneDensityOffset = 0.9f * Offset;

			auto AddLinesForLane = [this](const FZoneGraphStorage& Storage, const FZoneLaneData& Lane, const FColor Color, const float LineThickness, const FVector& Offset)
			{
				const FVector OffsetZ(0.f, 0.f, 0.1f);
				FVector PrevPoint = Storage.LanePoints[Lane.PointsBegin] + Offset;
				for (int32 PointIdx = Lane.PointsBegin + 1; PointIdx < Lane.PointsEnd; PointIdx++)
				{
					const FVector Point = Storage.LanePoints[PointIdx] + Offset;
					Lines.Add(FDebugRenderSceneProxy::FDebugLine(PrevPoint + OffsetZ, Point + OffsetZ, Color, LineThickness));
					PrevPoint = Point;
				}
			};

			for (int i = 0; i < Storage.Zones.Num(); i++)
			{
				const FZoneData& Zone = Storage.Zones[i];

				for (int32 LaneIndex = Zone.LanesBegin; LaneIndex < Zone.LanesEnd; ++LaneIndex)
				{
					const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
					// Render only lanes used by the crowd
					if (!Lane.Tags.Contains(CrowdTag))
					{
						continue;
					}

					const FZoneGraphLaneHandle LaneHandle(LaneIndex, Storage.DataHandle);
					const FCrowdBranchingLaneData* BranchingLaneData = MassCrowdSubsystem->GetCrowdBranchingLaneData(LaneHandle);
					const bool bIsBranchingLane = BranchingLaneData != nullptr;

					// states
					if (Settings->bDisplayStates)
					{
						TOptional<FZoneGraphCrowdLaneData> LaneData = MassCrowdSubsystem->GetCrowdLaneData(LaneHandle);
						const bool bIsOpened = LaneData.IsSet() ? LaneData.GetValue().GetState() == ECrowdLaneState::Opened : true;

						AddLinesForLane(Storage, Lane,
										bIsOpened ? Settings->OpenedLaneColor : Settings->ClosedLaneColor,
										bIsBranchingLane ? IntersectionLaneThickness : LaneThickness, Offset);
					}

					// densities
					if (Settings->bDisplayDensities)
					{
						const FZoneGraphTagMask& LaneTagMask = Storage.Lanes[LaneHandle.Index].Tags;
						const uint32 LaneMask = LaneTagMask.GetValue();
						const FZoneGraphTagMask LaneDensityMask(bIsBranchingLane ? BranchingLaneData->DensityMask : (LaneMask & MassCrowdSubsystem->GetDensityMask().GetValue()));

						if (LaneDensityMask.GetValue())
						{
							const FMassCrowdLaneDensityDesc* Descriptor = Settings->GetLaneDensities().FindByPredicate(
								[LaneDensityMask](const FMassCrowdLaneDensityDesc& Desc){ return LaneDensityMask.Contains(Desc.Tag);});

							if (Descriptor != nullptr)
							{
								AddLinesForLane(Storage, Lane,
												Descriptor->RenderColor,
												bIsBranchingLane ? IntersectionLaneDensityThickness : LaneDensityThickness, LaneDensityOffset);
							}
						}
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			// Disable dependency on Navigation show flag
			FGuardValue_Bitfield(const_cast<FSceneViewFamily*>(View->Family)->EngineShowFlags.Navigation, true);
			return FZoneGraphSceneProxy::GetViewRelevance(View);
		}

	private:
		const UMassCrowdLaneDataRenderingComponent* Component;
	};

	AZoneGraphData* ZoneGraph = Cast<AZoneGraphData>(GetOwner());
	return ZoneGraph ? new FMassCrowdLaneDataSceneProxy(*this, *ZoneGraph) : nullptr;
}

FBoxSphereBounds UMassCrowdLaneDataRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const AZoneGraphData* ZoneGraphData = Cast<AZoneGraphData>(GetOwner());
	return FBoxSphereBounds(ZoneGraphData != nullptr ? ZoneGraphData->GetBounds() : FBox(ForceInit));
}

void UMassCrowdLaneDataRenderingComponent::DebugDrawOnCanvas(UCanvas* Canvas, APlayerController*) const
{
	const UMassCrowdSubsystem* MassCrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(GetWorld());
	// The subsystems could be removed during world tear down
	if (MassCrowdSubsystem == nullptr)
	{
		return;
	}

	const UMassCrowdSettings* Settings = GetDefault<UMassCrowdSettings>();
	checkf(Settings, TEXT("Settings default object is always expected to be valid"));
	if (!Settings->bDisplayTrackingData)
	{
		return;
	}

	const AZoneGraphData* ZoneGraphData = Cast<AZoneGraphData>(GetOwner());
	const FZoneGraphStorage& ZoneStorage = ZoneGraphData->GetStorage();

	// Make sure we still have lane data since we can end up here when Crowd subsystem has already been torn down.
	if (!MassCrowdSubsystem->HasCrowdDataForZoneGraph(ZoneStorage.DataHandle))
	{
		return;
	}

	const FSceneView* View = Canvas->SceneView;
	TGuardValue<FColor> ColorGuard(Canvas->DrawColor, FColor::White);
	const UFont* Font = GEngine->GetSmallFont();
	const FZoneGraphTag CrowdTag = Settings->CrowdTag;

	// * A CachedMaxDrawDistance of 0 indicates that the primitive should not be culled by distance.
	const float MaxDrawDistance = CachedMaxDrawDistance > 0 ? CachedMaxDrawDistance : FLT_MAX;
	const FZoneGraphSceneProxy::FDrawDistances Distances = FZoneGraphSceneProxy::GetDrawDistances(MinDrawDistance, MaxDrawDistance);
	const FVector Origin = View->ViewMatrices.GetViewOrigin();

	for (const FZoneData& Zone : ZoneStorage.Zones)
	{
		const FZoneGraphSceneProxy::FZoneVisibility DrawInfo = FZoneGraphSceneProxy::CalculateZoneVisibility(Distances, Origin, Zone.Bounds.GetCenter());
		if (!DrawInfo.bVisible || !DrawInfo.bDetailsVisible)
		{
			continue;
		}

		for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
		{
			const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneIdx];
			// Render only lanes used by the crowd
			if (!Lane.Tags.Contains(CrowdTag))
			{
				continue;
			}

			const FZoneGraphLaneHandle LaneHandle(LaneIdx, ZoneStorage.DataHandle);
			const FCrowdWaitAreaData* WaitAreaData = MassCrowdSubsystem->GetCrowdWaitingAreaData(LaneHandle);
			const FCrowdTrackingLaneData* TrackingLaneData = MassCrowdSubsystem->GetCrowdTrackingLaneData(LaneHandle);
			if (WaitAreaData == nullptr && TrackingLaneData == nullptr)
			{
				continue;
			}
			checkf(TrackingLaneData != nullptr, TEXT("Tracking can exist without wait area data but not the opposite"));

			FZoneGraphLaneLocation CenterLoc;
			UE::ZoneGraph::Query::CalculateLocationAlongLaneFromRatio(ZoneStorage, LaneIdx, 0.5f, CenterLoc);
			const FVector Location = FVector(GetComponentTransform().TransformPosition(CenterLoc.Position));
			const FVector ScreenLoc = Canvas->Project(Location);

			if (WaitAreaData != nullptr)
			{
				Canvas->DrawText(Font, FString::Printf(TEXT("        %d/%d"), TrackingLaneData->NumEntitiesOnLane, WaitAreaData->GetNumSlots())
					, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
					, FloatCastChecked<float>(ScreenLoc.Y, UE::LWC::DefaultFloatPrecision));
			}
			else
			{
				Canvas->DrawText(Font, FString::Printf(TEXT(" %d"), TrackingLaneData->NumEntitiesOnLane)
					, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
					, FloatCastChecked<float>(ScreenLoc.Y, UE::LWC::DefaultFloatPrecision));
			}
		}
	}
}

#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST
