// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdMemberTrait.h"
#include "MassCrowdFragments.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdMemberTrait)

void UMassCrowdMemberTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddTag<FMassCrowdTag>();
	BuildContext.AddFragment<FMassCrowdLaneTrackingFragment>();	
}
