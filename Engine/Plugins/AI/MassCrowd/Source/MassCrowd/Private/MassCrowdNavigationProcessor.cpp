// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdNavigationProcessor.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassAIBehaviorTypes.h"
#include "MassCrowdSubsystem.h"
#include "MassCrowdFragments.h"
#include "MassEntityManager.h"
#include "MassMovementFragments.h"
#include "MassCrowdSettings.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "MassZoneGraphNavigationFragments.h"
#include "Annotations/ZoneGraphDisturbanceAnnotation.h"
#include "MassSimulationLOD.h"
#include "MassSignalSubsystem.h"

//----------------------------------------------------------------------//
// UMassCrowdLaneTrackingSignalProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdNavigationProcessor)
UMassCrowdLaneTrackingSignalProcessor::UMassCrowdLaneTrackingSignalProcessor()
{
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Behavior);
}

void UMassCrowdLaneTrackingSignalProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FMassCrowdLaneTrackingFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddSubsystemRequirement<UMassCrowdSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassCrowdLaneTrackingSignalProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);
	
	UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::CurrentLaneChanged);
}

void UMassCrowdLaneTrackingSignalProcessor::SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup&)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassCrowdSubsystem& MassCrowdSubsystem = Context.GetMutableSubsystemChecked<UMassCrowdSubsystem>();
		const TConstArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();
		const TArrayView<FMassCrowdLaneTrackingFragment> LaneTrackingList = Context.GetMutableFragmentView<FMassCrowdLaneTrackingFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];
			FMassCrowdLaneTrackingFragment& LaneTracking = LaneTrackingList[EntityIt];
			if (LaneTracking.TrackedLaneHandle != LaneLocation.LaneHandle)
			{
				MassCrowdSubsystem.OnEntityLaneChanged(Context.GetEntity(EntityIt), LaneTracking.TrackedLaneHandle, LaneLocation.LaneHandle);
				LaneTracking.TrackedLaneHandle = LaneLocation.LaneHandle;
			}
		}
	});
}

//----------------------------------------------------------------------//
// UMassCrowdLaneTrackingDestructor
//----------------------------------------------------------------------//
UMassCrowdLaneTrackingDestructor::UMassCrowdLaneTrackingDestructor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedTypes.Add(FMassCrowdLaneTrackingFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
}

void UMassCrowdLaneTrackingDestructor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FMassCrowdLaneTrackingFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddSubsystemRequirement<UMassCrowdSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassCrowdLaneTrackingDestructor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassCrowdSubsystem& MassCrowdSubsystem = Context.GetMutableSubsystemChecked<UMassCrowdSubsystem>();
		const TConstArrayView<FMassCrowdLaneTrackingFragment> LaneTrackingList = Context.GetFragmentView<FMassCrowdLaneTrackingFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassCrowdLaneTrackingFragment& LaneTracking = LaneTrackingList[EntityIt];
			if (LaneTracking.TrackedLaneHandle.IsValid())
			{
				MassCrowdSubsystem.OnEntityLaneChanged(Context.GetEntity(EntityIt), LaneTracking.TrackedLaneHandle, FZoneGraphLaneHandle());
			}
		}
	});
}

//----------------------------------------------------------------------//
// UMassCrowdDynamicObstacleProcessor
//----------------------------------------------------------------------//

UMassCrowdDynamicObstacleProcessor::UMassCrowdDynamicObstacleProcessor()
	: EntityQuery_Conditional(*this)
{
	bAutoRegisterWithProcessingPhases = true;

	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::UpdateAnnotationTags);
}

void UMassCrowdDynamicObstacleProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	ZoneGraphAnnotationSubsystem = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(Owner.GetWorld());
	checkf(ZoneGraphAnnotationSubsystem != nullptr, TEXT("UZoneGraphAnnotationSubsystem is mandatory when using this processor."));
}

void UMassCrowdDynamicObstacleProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery_Conditional.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddRequirement<FMassCrowdObstacleFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.SetChunkFilter(&FMassSimulationVariableTickChunkFragment::ShouldTickChunkThisFrame);
}

void UMassCrowdDynamicObstacleProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UWorld* World = EntityManager.GetWorld();
	const UMassCrowdSettings* CrowdSettings = GetDefault<UMassCrowdSettings>();
	checkf(CrowdSettings, TEXT("Settings default object is always expected to be valid"));

	EntityQuery_Conditional.ForEachEntityChunk(Context, [this, World, CrowdSettings](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
		const TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
		const TArrayView<FMassCrowdObstacleFragment> ObstacleDataList = Context.GetMutableFragmentView<FMassCrowdObstacleFragment>();
		const TConstArrayView<FMassSimulationVariableTickFragment> SimVariableTickList = Context.GetFragmentView<FMassSimulationVariableTickFragment>();

		const bool bHasVelocity = VelocityList.Num() > 0;
		const bool bHasVariableTick = (SimVariableTickList.Num() > 0);
		const float WorldDeltaTime = Context.GetDeltaTimeSeconds();

		const float ObstacleMovingDistanceTolerance = CrowdSettings->ObstacleMovingDistanceTolerance;
		const float ObstacleStoppingSpeedTolerance = CrowdSettings->ObstacleStoppingSpeedTolerance;
		const float ObstacleTimeToStop = CrowdSettings->ObstacleTimeToStop;
		const float ObstacleEffectRadius = CrowdSettings->ObstacleEffectRadius;
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// @todo: limit update frequency, this does not need to occur every frame
			const FVector Position = LocationList[EntityIt].GetTransform().GetLocation();
			const float Radius = RadiusList[EntityIt].Radius;
			FMassCrowdObstacleFragment& Obstacle = ObstacleDataList[EntityIt];
			const float DeltaTime = FMath::Max(KINDA_SMALL_NUMBER, bHasVariableTick ? SimVariableTickList[EntityIt].DeltaTime : WorldDeltaTime);

			UE_VLOG_LOCATION(this, LogMassNavigationObstacle, Display, Position, Radius, Obstacle.bIsMoving ? FColor::Green : FColor::Red, TEXT(""));

			if (Obstacle.bIsMoving)
			{
				// Calculate current speed based on velocity or last known position.
				const FVector::FReal CurrentSpeed = bHasVelocity ? VelocityList[EntityIt].Value.Length() : (FVector::Dist(Position, Obstacle.LastPosition) / DeltaTime);

				// Update position while moving, the stop logic will use the last position while check if the obstacles moves again.
				Obstacle.LastPosition = Position;

				// Keep track how long the obstacle has been almost stationary.
				if (CurrentSpeed < ObstacleStoppingSpeedTolerance)
				{
					Obstacle.TimeSinceStopped += DeltaTime;
				}
				else
				{
					Obstacle.TimeSinceStopped = 0.0f;
				}
				
				// If the obstacle has been almost stationary for a while, mark it as obstacle.
				if (Obstacle.TimeSinceStopped > ObstacleTimeToStop)
				{
					ensureMsgf(Obstacle.LaneObstacleID.IsValid() == false, TEXT("Obstacle should not have been set."));
						
					Obstacle.bIsMoving = false;
					Obstacle.LaneObstacleID = FMassLaneObstacleID::GetNextUniqueID();

					// Add an obstacle disturbance.
					FZoneGraphObstacleDisturbanceArea Disturbance;
					Disturbance.Position = Obstacle.LastPosition;
					Disturbance.Radius = ObstacleEffectRadius;
					Disturbance.ObstacleRadius = Radius;
					Disturbance.ObstacleID = Obstacle.LaneObstacleID;
					Disturbance.Action = EZoneGraphObstacleDisturbanceAreaAction::Add;
					ZoneGraphAnnotationSubsystem->SendEvent(Disturbance);
				}
			}
			else
			{
				Obstacle.TimeSinceStopped += DeltaTime;

				// If the obstacle moves outside movement tolerance, mark it as moving, and remove it as obstacle.
				if (FVector::Dist(Position, Obstacle.LastPosition) > ObstacleMovingDistanceTolerance)				
				{
					ensureMsgf(Obstacle.LaneObstacleID.IsValid(), TEXT("Obstacle should have been set."));

					FZoneGraphObstacleDisturbanceArea Disturbance;
					Disturbance.ObstacleID = Obstacle.LaneObstacleID;
					Disturbance.Action = EZoneGraphObstacleDisturbanceAreaAction::Remove;
					ZoneGraphAnnotationSubsystem->SendEvent(Disturbance);

					Obstacle.bIsMoving = true;
					Obstacle.TimeSinceStopped = 0.0f;
					Obstacle.LaneObstacleID = FMassLaneObstacleID();
				}
			}
		}
	});
}

//----------------------------------------------------------------------//
// UMassCrowdDynamicObstacleInitializer
//----------------------------------------------------------------------//
UMassCrowdDynamicObstacleInitializer::UMassCrowdDynamicObstacleInitializer()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedTypes.Add(FMassCrowdObstacleFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
}

void UMassCrowdDynamicObstacleInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassCrowdObstacleFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassCrowdDynamicObstacleInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UWorld* World = EntityManager.GetWorld();
	
	EntityQuery.ForEachEntityChunk(Context, [World](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TArrayView<FMassCrowdObstacleFragment> ObstacleDataList = Context.GetMutableFragmentView<FMassCrowdObstacleFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FVector Position = LocationList[EntityIt].GetTransform().GetLocation();
			FMassCrowdObstacleFragment& Obstacle = ObstacleDataList[EntityIt];

			Obstacle.LastPosition = Position;
			Obstacle.TimeSinceStopped = 0.0f;
			Obstacle.bIsMoving = true;
		}
	});
}


//----------------------------------------------------------------------//
// UMassCrowdDynamicObstacleDeinitializer
//----------------------------------------------------------------------//
UMassCrowdDynamicObstacleDeinitializer::UMassCrowdDynamicObstacleDeinitializer()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	ObservedTypes.Add(FMassCrowdObstacleFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
}

void UMassCrowdDynamicObstacleDeinitializer::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	UWorld* World = Owner.GetWorld();
	ZoneGraphAnnotationSubsystem = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(World);
	checkf(ZoneGraphAnnotationSubsystem != nullptr || (World && World->WorldType == EWorldType::Inactive)
		, TEXT("UZoneGraphAnnotationSubsystem is mandatory when using this processor."));
}

void UMassCrowdDynamicObstacleDeinitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassCrowdObstacleFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassCrowdDynamicObstacleDeinitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TArrayView<FMassCrowdObstacleFragment> ObstacleDataList = Context.GetMutableFragmentView<FMassCrowdObstacleFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassCrowdObstacleFragment& Obstacle = ObstacleDataList[EntityIt];

			if (Obstacle.LaneObstacleID.IsValid())
			{
				FZoneGraphObstacleDisturbanceArea Disturbance;
				Disturbance.ObstacleID = Obstacle.LaneObstacleID;
				Disturbance.Action = EZoneGraphObstacleDisturbanceAreaAction::Remove;
				ZoneGraphAnnotationSubsystem->SendEvent(Disturbance);

				// Reset obstacle
				Obstacle = FMassCrowdObstacleFragment();
			}
		}
	});
}
