// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdReplicator.h"
#include "MassCrowdBubble.h"
#include "MassExecutionContext.h"

//----------------------------------------------------------------------//
//  UMassCrowdReplicator
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdReplicator)
void UMassCrowdReplicator::AddRequirements(FMassEntityQuery& EntityQuery)
{
	FMassReplicationProcessorPositionYawHandler::AddRequirements(EntityQuery);
	FMassReplicationProcessorPathHandler::AddRequirements(EntityQuery);
}

void UMassCrowdReplicator::ProcessClientReplication(FMassExecutionContext& Context, FMassReplicationContext& ReplicationContext)
{
#if UE_REPLICATION_COMPILE_SERVER_CODE

	FMassReplicationProcessorPathHandler PathHandler;
	FMassReplicationProcessorPositionYawHandler PositionYawHandler;
	FMassReplicationSharedFragment* RepSharedFrag = nullptr;

	auto CacheViewsCallback = [&RepSharedFrag, &PathHandler, &PositionYawHandler](FMassExecutionContext& Context)
	{
		PathHandler.CacheFragmentViews(Context);
		PositionYawHandler.CacheFragmentViews(Context);
		RepSharedFrag = &Context.GetMutableSharedFragment<FMassReplicationSharedFragment>();
		check(RepSharedFrag);
	};

	auto AddEntityCallback = [&RepSharedFrag, &PathHandler, &PositionYawHandler](FMassExecutionContext& Context, const int32 EntityIdx, FReplicatedCrowdAgent& InReplicatedAgent, const FMassClientHandle ClientHandle)->FMassReplicatedAgentHandle
	{
		AMassCrowdClientBubbleInfo& CrowdBubbleInfo = RepSharedFrag->GetTypedClientBubbleInfoChecked<AMassCrowdClientBubbleInfo>(ClientHandle);

		PathHandler.AddEntity(EntityIdx, InReplicatedAgent.GetReplicatedPathDataMutable());
		PositionYawHandler.AddEntity(EntityIdx, InReplicatedAgent.GetReplicatedPositionYawDataMutable());

		return CrowdBubbleInfo.GetCrowdSerializer().Bubble.AddAgent(Context.GetEntity(EntityIdx), InReplicatedAgent);
	};

	auto ModifyEntityCallback = [&RepSharedFrag, &PathHandler](FMassExecutionContext& Context, const int32 EntityIdx, const EMassLOD::Type LOD, const double Time, const FMassReplicatedAgentHandle Handle, const FMassClientHandle ClientHandle)
	{
		AMassCrowdClientBubbleInfo& CrowdBubbleInfo = RepSharedFrag->GetTypedClientBubbleInfoChecked<AMassCrowdClientBubbleInfo>(ClientHandle);

		FMassCrowdClientBubbleHandler& Bubble = CrowdBubbleInfo.GetCrowdSerializer().Bubble;

		const bool bLastClient = RepSharedFrag->CachedClientHandles.Last() == ClientHandle;
		PathHandler.ModifyEntity<FCrowdFastArrayItem>(Handle, EntityIdx, Bubble.GetPathHandlerMutable(), bLastClient);

		// Don't call the PositionYawHandler here as we currently only replicate the position and yaw when we add an entity to Mass
	};

	auto RemoveEntityCallback = [&RepSharedFrag](FMassExecutionContext& Context, const FMassReplicatedAgentHandle Handle, const FMassClientHandle ClientHandle)
	{
		AMassCrowdClientBubbleInfo& CrowdBubbleInfo = RepSharedFrag->GetTypedClientBubbleInfoChecked<AMassCrowdClientBubbleInfo>(ClientHandle);

		CrowdBubbleInfo.GetCrowdSerializer().Bubble.RemoveAgentChecked(Handle);
	};

	CalculateClientReplication<FCrowdFastArrayItem>(Context, ReplicationContext, CacheViewsCallback, AddEntityCallback, ModifyEntityCallback, RemoveEntityCallback);
#endif // UE_REPLICATION_COMPILE_SERVER_CODE
}
