// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdRepresentationActorManagement.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/CapsuleComponent.h"
#include "Engine/World.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "GameFramework/Character.h"
#include "MassAgentComponent.h"
#include "MassRepresentationSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdRepresentationActorManagement)

void UMassCrowdRepresentationActorManagement::SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx, FMassCommandBuffer& CommandBuffer) const
{
	Super::SetActorEnabled(EnabledType, Actor, EntityIdx, CommandBuffer);

	const bool bEnabled = EnabledType != EMassActorEnabledType::Disabled;

	USkeletalMeshComponent* SkeletalMeshComponent = Actor.FindComponentByClass<USkeletalMeshComponent>();
	if (SkeletalMeshComponent)
	{
		// Enable/disable the ticking and visibility of SkeletalMesh and its children
		SkeletalMeshComponent->SetVisibility(bEnabled);
		SkeletalMeshComponent->SetComponentTickEnabled(bEnabled);
		const TArray<USceneComponent*>& AttachedChildren = SkeletalMeshComponent->GetAttachChildren();
		if (AttachedChildren.Num() > 0)
		{
			TInlineComponentArray<USceneComponent*, NumInlinedActorComponents> ComponentStack;

			ComponentStack.Append(AttachedChildren);
			while (ComponentStack.Num() > 0)
			{
				USceneComponent* const CurrentComp = ComponentStack.Pop(EAllowShrinking::No);
				if (CurrentComp)
				{
					ComponentStack.Append(CurrentComp->GetAttachChildren());
					CurrentComp->SetVisibility(bEnabled);
					if (bEnabled)
					{
						// Re-enable only if it was enabled at startup
						CurrentComp->SetComponentTickEnabled(CurrentComp->PrimaryComponentTick.bStartWithTickEnabled);
					}
					else
					{
						CurrentComp->SetComponentTickEnabled(false);
					}
				}
			}
		}
	}

	// Enable/disable the ticking of CharacterMovementComponent as well
	ACharacter* Character = Cast<ACharacter>(&Actor);
	UCharacterMovementComponent* MovementComp = Character != nullptr ? Character->GetCharacterMovement() : nullptr;
	if (MovementComp != nullptr)
	{
		MovementComp->SetComponentTickEnabled(bEnabled);
	}

	// when we "suspend" the puppet actor we need to let the agent subsystem know by unregistering the agent component
	// associated with the actor. This will result in removing all the puppet-actor-specific fragments which in turn
	// will exclude the owner entity from being processed by puppet-specific processors (usually translators).
	if (UMassAgentComponent* AgentComp = Actor.FindComponentByClass<UMassAgentComponent>())
	{
		if (AgentComp->IsPuppet())
		{
			AgentComp->PausePuppet(!bEnabled);
		}
	}
}

AActor* UMassCrowdRepresentationActorManagement::GetOrSpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem
	, FMassEntityManager& EntityManager, const FMassEntityHandle MassAgent, const FTransform& Transform
	, const int16 TemplateActorIndex, FMassActorSpawnRequestHandle& SpawnRequestHandle, const float Priority) const
{
	FTransform RootTransform = Transform;
	
	if (const AActor* DefaultActor = RepresentationSubsystem.GetTemplateActorClass(TemplateActorIndex).GetDefaultObject())
	{
		if (const UCapsuleComponent* CapsuleComp = DefaultActor->FindComponentByClass<UCapsuleComponent>())
		{
			RootTransform.AddToTranslation(FVector(0.0f, 0.0f, CapsuleComp->GetScaledCapsuleHalfHeight()));
		}
	}

	return Super::GetOrSpawnActor(RepresentationSubsystem, EntityManager, MassAgent, RootTransform, TemplateActorIndex, SpawnRequestHandle, Priority);
}

void UMassCrowdRepresentationActorManagement::TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const
{
	FTransform RootTransform = Transform;

	if (const UCapsuleComponent* CapsuleComp = Actor.FindComponentByClass<UCapsuleComponent>())
	{
		const FVector HalfHeight(0.0f, 0.0f, CapsuleComp->GetScaledCapsuleHalfHeight());
		RootTransform.AddToTranslation(HalfHeight);
		const FVector RootLocation = RootTransform.GetLocation();
		const FVector SweepOffset(0.0f, 0.0f, 20.0f);
		const FVector Start = RootLocation + SweepOffset;
		const FVector End = RootLocation - SweepOffset;
		FCollisionQueryParams Params;
		Params.AddIgnoredActor(&Actor);
		FHitResult OutHit;
		if (Actor.GetWorld()->SweepSingleByChannel(OutHit, Start, End, Transform.GetRotation(), CapsuleComp->GetCollisionObjectType(), CapsuleComp->GetCollisionShape(), Params))
		{
			RootTransform.SetLocation(OutHit.Location);
		}
	}
	Super::TeleportActor(RootTransform, Actor, CommandBuffer);
}
