// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdRepresentationSubsystem.h"
#include "MassCrowdSpawnerSubsystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdRepresentationSubsystem)

void UMassCrowdRepresentationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency(UMassCrowdSpawnerSubsystem::StaticClass());

	Super::Initialize(Collection);

	ActorSpawnerSubsystem = UWorld::GetSubsystem<UMassCrowdSpawnerSubsystem>(GetWorld());
}
