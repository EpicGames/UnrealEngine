// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdServerRepresentationLODProcessor.h"
#include "MassCommonTypes.h"
#include "MassCommonFragments.h"
#include "MassCrowdFragments.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "MassRepresentationFragments.h"
#include "MassEntityManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdServerRepresentationLODProcessor)

namespace UE::MassCrowd
{
	int32 bDebugCrowdServerRepresentationLOD = 0;
	FAutoConsoleVariableRef CVarDebugServerRepresentationLODTest(TEXT("mass.debug.CrowdServerRepresentationLOD"), bDebugCrowdServerRepresentationLOD, TEXT("Debug Crowd ServerRepresentation LOD"), ECVF_Cheat);
} // UE::MassCrowd

UMassCrowdServerRepresentationLODProcessor::UMassCrowdServerRepresentationLODProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::Server;

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LOD;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);

	LODDistance[EMassLOD::High] = 0.0f;
	LODDistance[EMassLOD::Medium] = 5000.0f;
	LODDistance[EMassLOD::Low] = 5000.0f;
	LODDistance[EMassLOD::Off] = 5000.0f;
	
	LODMaxCount[EMassLOD::High] = 50;
	LODMaxCount[EMassLOD::Medium] = 0;
	LODMaxCount[EMassLOD::Low] = 0;
	LODMaxCount[EMassLOD::Off] = INT_MAX;
}

void UMassCrowdServerRepresentationLODProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassViewerInfoFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassCrowdServerRepresentationLODProcessor::InitializeInternal(UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	LODCalculator.Initialize(LODDistance, BufferHysteresisOnDistancePercentage / 100.0f, LODMaxCount);

	Super::InitializeInternal(InOwner, EntityManager);
}

void UMassCrowdServerRepresentationLODProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CrowdServerRepresentationLOD)

	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	const TArray<FViewerInfo>& Viewers = LODSubsystem.GetViewers();
	LODCalculator.PrepareExecution(Viewers);
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CalculateLOD)
		
		EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
			const TArrayView<FMassRepresentationLODFragment> RepresentationLODFragments = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
			LODCalculator.CalculateLOD(Context, ViewersInfoList, RepresentationLODFragments);
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AdjustDistancesAndLODFromCount)
		
		if (LODCalculator.AdjustDistancesFromCount())
		{
			EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
			{
				const TConstArrayView<FMassViewerInfoFragment> ViewersInfoList = Context.GetFragmentView<FMassViewerInfoFragment>();
				const TArrayView<FMassRepresentationLODFragment> RepresentationLODFragments = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
				LODCalculator.AdjustLODFromCount(Context, ViewersInfoList, RepresentationLODFragments);
			});
		}
	}

#if WITH_MASSGAMEPLAY_DEBUG
	// Optional debug display
	if (UE::MassCrowd::bDebugCrowdServerRepresentationLOD)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayLOD)
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			EntityQuery.ForEachEntityChunk(Context, [this, DebugDrawingWorld](FMassExecutionContext& Context)	
			{
				const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
				const TConstArrayView<FMassRepresentationLODFragment> RepresentationLODFragments = Context.GetFragmentView<FMassRepresentationLODFragment>();
				LODCalculator.DebugDisplayLOD(Context, RepresentationLODFragments, LocationList, DebugDrawingWorld);
			});
		}
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}
