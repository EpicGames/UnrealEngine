// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdServerRepresentationProcessor.h"
#include "MassCrowdFragments.h"
#include "MassLODTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdServerRepresentationProcessor)

UMassCrowdServerRepresentationProcessor::UMassCrowdServerRepresentationProcessor()
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::Server;

	bAutoRegisterWithProcessingPhases = true;
	bRequiresGameThreadExecution = true;

	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
}

void UMassCrowdServerRepresentationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
}
