// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdServerRepresentationTrait.h"
#include "MassEntityManager.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCrowdRepresentationSubsystem.h"
#include "MassCrowdRepresentationActorManagement.h"
#include "MassActorSubsystem.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdServerRepresentationTrait)


UMassCrowdServerRepresentationTrait::UMassCrowdServerRepresentationTrait()
{
	Params.RepresentationActorManagementClass = UMassCrowdRepresentationActorManagement::StaticClass();
	Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::HighResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::None;
	Params.LODRepresentation[EMassLOD::Low] = EMassRepresentationType::None;
	Params.LODRepresentation[EMassLOD::Off] = EMassRepresentationType::None;
	Params.bKeepLowResActors = false;
	Params.bKeepActorExtraFrame = false;
	Params.bSpreadFirstVisualizationUpdate = false;
	Params.WorldPartitionGridNameContainingCollision = NAME_None;
	Params.NotVisibleUpdateRate = 0.5f;
}

void UMassCrowdServerRepresentationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	// This should only be ran on NM_DedicatedServer network mode
	if (!World.IsNetMode(NM_DedicatedServer) && !BuildContext.IsInspectingData())
	{
		return;
	}
	
	BuildContext.RequireFragment<FMassViewerInfoFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassActorFragment>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	UMassCrowdRepresentationSubsystem* RepresentationSubsystem = World.GetSubsystem<UMassCrowdRepresentationSubsystem>();
	check(RepresentationSubsystem || BuildContext.IsInspectingData());

	FMassRepresentationSubsystemSharedFragment SubsystemSharedFragment;
	SubsystemSharedFragment.RepresentationSubsystem = RepresentationSubsystem;
	FSharedStruct SubsystemFragment = EntityManager.GetOrCreateSharedFragment(SubsystemSharedFragment);
	BuildContext.AddSharedFragment(SubsystemFragment);

	FConstSharedStruct ParamsFragment = EntityManager.GetOrCreateConstSharedFragment(Params);
	ParamsFragment.Get<const FMassRepresentationParameters>().ComputeCachedValues();
	BuildContext.AddConstSharedFragment(ParamsFragment);

	FMassRepresentationFragment& RepresentationFragment = BuildContext.AddFragment_GetRef<FMassRepresentationFragment>();
	RepresentationFragment.StaticMeshDescHandle = FStaticMeshInstanceVisualizationDescHandle();
	if (!BuildContext.IsInspectingData())
	{
		RepresentationFragment.HighResTemplateActorIndex = TemplateActor.Get() ? RepresentationSubsystem->FindOrAddTemplateActor(TemplateActor.Get()) : INDEX_NONE;
	}
	RepresentationFragment.LowResTemplateActorIndex = INDEX_NONE;

	BuildContext.AddFragment<FMassRepresentationLODFragment>();

	// @todo figure out if this chunk fragment is really needed?
	BuildContext.AddChunkFragment<FMassVisualizationChunkFragment>();
}
