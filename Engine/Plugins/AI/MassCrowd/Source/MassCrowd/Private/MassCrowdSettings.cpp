// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdSettings)

#if WITH_EDITOR
void UMassCrowdSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty != nullptr && Property != nullptr)
	{
		const FName MemberPropertyName = MemberProperty->GetFName();
		const FName PropertyName = Property->GetFName();
		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, LaneDensities) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, CrowdTag) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, CrossingTag) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, SlotSize) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, SlotOffset))
		{
			OnMassCrowdLaneDataSettingsChanged.Broadcast();
		}

		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, LaneBaseLineThickness) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, LaneRenderZOffset) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, IntersectionLaneScaleFactor) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, LaneDensityScaleFactor) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, OpenedLaneColor) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, ClosedLaneColor) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, bDisplayTrackingData) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, bDisplayStates) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, bDisplayDensities) ||
			(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMassCrowdSettings, LaneDensities) && PropertyName == GET_MEMBER_NAME_CHECKED(FMassCrowdLaneDensityDesc, RenderColor)))
		{
			OnMassCrowdLaneRenderSettingsChanged.Broadcast();
		}
	}
}
#endif // WITH_EDITOR
