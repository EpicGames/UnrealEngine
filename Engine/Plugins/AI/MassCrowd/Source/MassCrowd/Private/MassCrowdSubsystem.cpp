// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdSubsystem.h"
#include "MassSimulationSubsystem.h"
#include "MassCrowdBubble.h"
#include "MassCrowdSettings.h"
#include "Subsystems/Subsystem.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphCrowdLaneAnnotations.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSubsystem.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdSubsystem)

#if WITH_MASSGAMEPLAY_DEBUG
namespace UE::MassCrowdDebug
{
bool bForceFillIntersectionLanes = false;
FAutoConsoleVariableRef CVars[] =
{
	FAutoConsoleVariableRef(TEXT("ai.mass.CrowdForceFillIntersections"),
							UE::MassCrowdDebug::bForceFillIntersectionLanes,
							TEXT("Modify lane selection to always choose a non full closed lane (if any)."), ECVF_Cheat)
};

#if WITH_EDITOR
static FAutoConsoleCommandWithWorld RebuildCmd(
	TEXT("ai.mass.CrowdRebuildLaneData"),
	TEXT("Clears and rebuilds lane and intersection data for registered zone graphs using MassCrowd settings."),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
	{
		if (UMassCrowdSubsystem* MassCrowdSubsystem = World != nullptr ? World->GetSubsystem<UMassCrowdSubsystem>() : nullptr)
		{
			MassCrowdSubsystem->RebuildLaneData();
		}
	}));
#endif // WITH_EDITOR
}// namespace UE::MassCrowdDebug
#endif // WITH_MASSGAMEPLAY_DEBUG

//----------------------------------------------------------------------//
// UMassCrowdSubsystem
//----------------------------------------------------------------------//
void UMassCrowdSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	Collection.InitializeDependency<UMassSimulationSubsystem>();
	ZoneGraphSubsystem = Collection.InitializeDependency<UZoneGraphSubsystem>();
	ZoneGraphAnnotationSubsystem = Collection.InitializeDependency<UZoneGraphAnnotationSubsystem>();

	// Cache settings
	MassCrowdSettings = GetDefault<UMassCrowdSettings>();

	// Compute the combined mask regrouping all possible densities
	// This must be updated before registering zone graphs
	UpdateDensityMask();

	// Register existing data.
	for (const FRegisteredZoneGraphData& Registered : ZoneGraphSubsystem->GetRegisteredZoneGraphData())
	{
		if (Registered.bInUse && Registered.ZoneGraphData != nullptr)
		{
			PostZoneGraphDataAdded(Registered.ZoneGraphData);
		}
	}

	OnPostZoneGraphDataAddedHandle = UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.AddUObject(this, &UMassCrowdSubsystem::PostZoneGraphDataAdded);
	OnPreZoneGraphDataRemovedHandle = UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.AddUObject(this, &UMassCrowdSubsystem::PreZoneGraphDataRemoved);

#if WITH_EDITOR
	// Track density settings changes
	OnMassCrowdSettingsChangedHandle = MassCrowdSettings->OnMassCrowdLaneDataSettingsChanged.AddLambda([this]()
	{
		UpdateDensityMask();
		RebuildLaneData();
	});

	OnZoneGraphDataBuildDoneHandle = UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.AddLambda([this](const FZoneGraphBuildData& /*BuildData*/)
	{
		UpdateDensityMask();
		RebuildLaneData();
	});
#endif

	HandleLateCreation();

	OverrideSubsystemTraits<UMassCrowdSubsystem>(Collection);
}

void UMassCrowdSubsystem::Deinitialize()
{
#if WITH_EDITOR
	checkf(MassCrowdSettings, TEXT("UMassCrowdSettings CDO should have been cached in Initialize"));
	MassCrowdSettings->OnMassCrowdLaneDataSettingsChanged.Remove(OnMassCrowdSettingsChangedHandle);

	UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.Remove(OnZoneGraphDataBuildDoneHandle);
#endif

	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Remove(OnPostZoneGraphDataAddedHandle);
	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Remove(OnPreZoneGraphDataRemovedHandle);

	Super::Deinitialize();
}

void UMassCrowdSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UMassReplicationSubsystem* ReplicationSubsystem = UWorld::GetSubsystem<UMassReplicationSubsystem>(GetWorld());

	check(ReplicationSubsystem);
	ReplicationSubsystem->RegisterBubbleInfoClass(AMassCrowdClientBubbleInfo::StaticClass());
}

void UMassCrowdSubsystem::PostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData)
{
	const UWorld* World = GetWorld();

	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != World)
	{
		return;
	}

	const FString WorldName = World->GetName();

	const FZoneGraphStorage& Storage = ZoneGraphData->GetStorage();
	const int32 Index = Storage.DataHandle.Index;

	UE_VLOG_UELOG(this, LogMassNavigation, Verbose, TEXT("%s adding data %d/%d"), *WorldName, Storage.DataHandle.Index, Storage.DataHandle.Generation);

	if (Index >= RegisteredLaneData.Num())
	{
		RegisteredLaneData.SetNum(Index + 1);
	}

	FRegisteredCrowdLaneData& LaneData = RegisteredLaneData[Index];
	if (LaneData.DataHandle != Storage.DataHandle)
	{
		// Initialize lane data if here the first time.
		BuildLaneData(LaneData, Storage);
	}
}

void UMassCrowdSubsystem::PreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	const FZoneGraphStorage& Storage = ZoneGraphData->GetStorage();
	const int32 Index = Storage.DataHandle.Index;

	if (!RegisteredLaneData.IsValidIndex(Index))
	{
		return;
	}

	FRegisteredCrowdLaneData& LaneData = RegisteredLaneData[Index];
	LaneData.Reset();
}

void UMassCrowdSubsystem::BuildLaneData(FRegisteredCrowdLaneData& LaneData, const FZoneGraphStorage& Storage)
{
	LaneData.DataHandle = Storage.DataHandle;
	LaneData.CrowdLaneDataArray.Reset(Storage.Lanes.Num());
	LaneData.CrowdLaneDataArray.SetNum(Storage.Lanes.Num());

	// Graph may contain lanes that won't be used by the crowd so let's filter them at least for the intersection data.
	// regular lane data is always created to preserve fast index based access.
	const FZoneGraphTag CrowdTag = MassCrowdSettings->CrowdTag;
	const FZoneGraphTag CrossingTag = MassCrowdSettings->CrossingTag;

	checkf(ZoneGraphAnnotationSubsystem != nullptr, TEXT("ZoneGraphAnnotationSubsystem should be initialized from the subsystem collection dependencies."));

	TArray<FZoneGraphLinkedLane> Links;
	for (int32 LaneIndex = 0; LaneIndex < Storage.Lanes.Num(); ++LaneIndex)
	{
		const FZoneLaneData& ZoneLaneData = Storage.Lanes[LaneIndex];
		if (!ZoneLaneData.Tags.Contains(CrowdTag))
		{
			continue;
		}

		UE::ZoneGraph::Query::GetLinkedLanes(Storage, LaneIndex, EZoneLaneLinkType::Incoming, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, Links);

		// We only need tracking data for crossing lanes ond their incoming lanes
		if (ZoneLaneData.Tags.Contains(CrossingTag))
		{
			CreateTrackingData(LaneIndex, Storage);

			// Close all crossings by default.
			ZoneGraphAnnotationSubsystem->SendEvent(FZoneGraphCrowdLaneStateChangeEvent({LaneIndex, Storage.DataHandle}, ECrowdLaneState::Closed));

			const int32 WaitAreaIndex = LaneData.WaitAreas.AddDefaulted();
			FCrowdWaitAreaData& WaitingArea = LaneData.WaitAreas[WaitAreaIndex];
			CreateWaitSlots(LaneIndex, WaitingArea, Storage);
			for (const FZoneGraphLinkedLane& Link : Links)
			{
				FCrowdTrackingLaneData& TrackingData = CreateTrackingData(Link.DestLane.Index, Storage);
				TrackingData.WaitAreaIndex = WaitAreaIndex;

				WaitingArea.ConnectedLanes.Add(Link.DestLane);
			}
		}

		// We only need branching data for merging/splitting lanes since they require per lane density
		const bool bRequiresBranchingData = (Links.Num() > 1);
		if (bRequiresBranchingData)
		{
			for (const FZoneGraphLinkedLane& Link : Links)
			{
				CreateBranchingData(Link.DestLane.Index, Storage);
			}
		}
 	}
}

TOptional<FZoneGraphCrowdLaneData> UMassCrowdSubsystem::GetCrowdLaneData(const FZoneGraphLaneHandle LaneHandle) const
{
	TOptional<FZoneGraphCrowdLaneData> LaneData;
	if (const FZoneGraphCrowdLaneData* MutableLaneData = const_cast<UMassCrowdSubsystem*>(this)->GetMutableCrowdLaneData(LaneHandle))
	{
		LaneData = *MutableLaneData;
	}
	return LaneData;
}

FZoneGraphCrowdLaneData* UMassCrowdSubsystem::GetMutableCrowdLaneData(const FZoneGraphLaneHandle LaneHandle)
{
	if (!ensureMsgf(LaneHandle.IsValid(), TEXT("Invalid lane handle: returning an invalid entry.")))
	{
		return nullptr;
	}

	const int32 Index = LaneHandle.DataHandle.Index;
	if (!ensureMsgf(RegisteredLaneData.IsValidIndex(Index), TEXT("Invalid lane handle index: returning an invalid entry.")))
	{
		return nullptr;
	}

	FRegisteredCrowdLaneData& LaneData = RegisteredLaneData[Index];
	if (!ensureMsgf(LaneData.DataHandle == LaneHandle.DataHandle, TEXT("Mismatching data handle: returning an invalid entry.")))
	{
		return nullptr;
	}

	return &(LaneData.CrowdLaneDataArray[LaneHandle.Index]);
}

const FCrowdTrackingLaneData* UMassCrowdSubsystem::GetCrowdTrackingLaneData(const FZoneGraphLaneHandle LaneHandle) const
{
	if (!ensureMsgf(LaneHandle.IsValid(), TEXT("Invalid lane handle: returning a null entry.")))
	{
		return nullptr;
	}

	checkf(RegisteredLaneData.IsValidIndex(LaneHandle.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	const FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[LaneHandle.DataHandle.Index];
	return CrowdLaneData.LaneToTrackingDataLookup.Find(LaneHandle.Index);
}

const FCrowdBranchingLaneData* UMassCrowdSubsystem::GetCrowdBranchingLaneData(const FZoneGraphLaneHandle LaneHandle) const
{
	if (!ensureMsgf(LaneHandle.IsValid(), TEXT("Invalid lane handle: returning a null entry.")))
	{
		return nullptr;
	}

	checkf(RegisteredLaneData.IsValidIndex(LaneHandle.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	const FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[LaneHandle.DataHandle.Index];
	return CrowdLaneData.LaneToBranchingDataLookup.Find(LaneHandle.Index);
}

const FCrowdWaitAreaData* UMassCrowdSubsystem::GetCrowdWaitingAreaData(const FZoneGraphLaneHandle LaneHandle) const
{
	if (!ensureMsgf(LaneHandle.IsValid(), TEXT("Invalid lane handle: returning a null entry.")))
	{
		return nullptr;
	}

	checkf(RegisteredLaneData.IsValidIndex(LaneHandle.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	const FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[LaneHandle.DataHandle.Index];
	const FCrowdTrackingLaneData* TrackingData = CrowdLaneData.LaneToTrackingDataLookup.Find(LaneHandle.Index);
	
	return (TrackingData && TrackingData->WaitAreaIndex != INDEX_NONE) ? &CrowdLaneData.WaitAreas[TrackingData->WaitAreaIndex] : nullptr;
}

#if WITH_EDITOR
void UMassCrowdSubsystem::RebuildLaneData()
{
	if (ZoneGraphSubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassNavigation, Warning, TEXT("%s called before ZoneGraphSubsystem is set. Nothing to do."), ANSI_TO_TCHAR(__FUNCTION__));
		return;
	};

	UWorld* World = GetWorld();
	if (World != nullptr && World->IsGameWorld())
	{
		UE_VLOG_UELOG(this, LogMassNavigation, Warning, TEXT("%s is not supported on game world since data is in use."), ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}

	for (FRegisteredCrowdLaneData& LaneData : RegisteredLaneData)
	{
		LaneData.LaneToTrackingDataLookup.Reset();
		LaneData.LaneToBranchingDataLookup.Reset();
		LaneData.WaitAreas.Reset();
		const FZoneGraphStorage* Storage = ZoneGraphSubsystem->GetZoneGraphStorage(LaneData.DataHandle);
		if (Storage)
		{
			BuildLaneData(LaneData, *Storage);
		}
	}
}
#endif // WITH_EDITOR

bool UMassCrowdSubsystem::HasCrowdDataForZoneGraph(const FZoneGraphDataHandle DataHandle) const
{
	if (!DataHandle.IsValid())
	{
		return false;
	}

	const int32 Index = DataHandle.Index;
	if (!RegisteredLaneData.IsValidIndex(Index))
	{
		return false;
	}

	const FRegisteredCrowdLaneData& LaneData = RegisteredLaneData[Index];
	if (LaneData.DataHandle != DataHandle)
	{
		return false;
	}

	return true;
}

const FRegisteredCrowdLaneData* UMassCrowdSubsystem::GetCrowdData(const FZoneGraphDataHandle DataHandle) const
{
	if (!ensureMsgf(DataHandle.IsValid(), TEXT("Requesting crowd data using an invalid handle.")))
	{
		return nullptr;
	}

	const int32 Index = DataHandle.Index;
	if (!ensureMsgf(RegisteredLaneData.IsValidIndex(Index),
		TEXT("Requesting crowd data from a valid handle but associated data was not generated (e.g. Graph registration was not processed).")))
	{
		return nullptr;
	}

	const FRegisteredCrowdLaneData& LanesData = RegisteredLaneData[Index];
	if (!ensureMsgf(LanesData.DataHandle == DataHandle,
		TEXT("Mismatch between the graph handle stored in the associated crowd data and the provided handle (e.g. inconsistent registration/unregistration).")))
	{
		return nullptr;
	}

	return &LanesData;
}

void UMassCrowdSubsystem::OnEntityLaneChanged(const FMassEntityHandle Entity, const FZoneGraphLaneHandle PreviousLaneHandle, const FZoneGraphLaneHandle CurrentLaneHandle)
{
	const bool bPreviousLocationValid = PreviousLaneHandle.IsValid();
	const bool bCurrentLocationValid = CurrentLaneHandle.IsValid();

	if (!bPreviousLocationValid && !bCurrentLocationValid)
	{
		return;
	}

	const int32 DataIndex = bPreviousLocationValid ? PreviousLaneHandle.DataHandle.Index : CurrentLaneHandle.DataHandle.Index;
	checkf(RegisteredLaneData.IsValidIndex(DataIndex), TEXT("Storage must have been allocated before creating lane data"));
	FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[DataIndex];

	if (bPreviousLocationValid)
	{
		const int32 PrevLaneIndex = PreviousLaneHandle.Index;

		if (FCrowdTrackingLaneData* PreviousTrackingLaneData = CrowdLaneData.LaneToTrackingDataLookup.Find(PrevLaneIndex))
		{
			OnExitTrackedLane(Entity, PrevLaneIndex, *PreviousTrackingLaneData);
		}
		
		UE_VLOG_UELOG(this, LogMassNavigation, Verbose, TEXT("Entity [%s] exits lane %d"), *Entity.DebugGetDescription(), PrevLaneIndex);
	}

	if (bCurrentLocationValid)
	{
		const int32 CurLaneIndex = CurrentLaneHandle.Index;

		if (FCrowdTrackingLaneData* CurrentTrackingLaneData = CrowdLaneData.LaneToTrackingDataLookup.Find(CurLaneIndex))
		{
			OnEnterTrackedLane(Entity, CurLaneIndex, *CurrentTrackingLaneData);
		}

		UE_VLOG_UELOG(this, LogMassNavigation, Verbose, TEXT("Entity [%s] enters lane %d"), *Entity.DebugGetDescription(), CurLaneIndex);
	}
}

float UMassCrowdSubsystem::GetDensityWeight(const FZoneGraphLaneHandle LaneHandle, const FZoneGraphTagMask LaneTagMask) const
{
	const FCrowdBranchingLaneData* BranchingLaneData = GetCrowdBranchingLaneData(LaneHandle);
	const uint32 LaneMask = LaneTagMask.GetValue();
	const FZoneGraphTagMask LaneDensityMask((BranchingLaneData != nullptr) ? BranchingLaneData->DensityMask : (LaneMask & DensityMask.GetValue()));

	float Weight = FMassCrowdLaneDensityDesc::DefaultWeight;
	if (LaneDensityMask.GetValue())
	{
		for (const FMassCrowdLaneDensityDesc& DensityDescriptor : MassCrowdSettings->GetLaneDensities())
		{
			if (LaneDensityMask.Contains(DensityDescriptor.Tag))
			{
				Weight = DensityDescriptor.Weight;
				break;
			}
		}
	}

	return Weight;
}

ECrowdLaneState UMassCrowdSubsystem::GetLaneState(const FZoneGraphLaneHandle LaneHandle) const
{
	const TOptional<FZoneGraphCrowdLaneData> LaneData = GetCrowdLaneData(LaneHandle);
	return LaneData.IsSet() ? LaneData.GetValue().GetState() : ECrowdLaneState::Opened;
}

bool UMassCrowdSubsystem::SetLaneState(const FZoneGraphLaneHandle LaneHandle, ECrowdLaneState NewState)
{
	if (!LaneHandle.IsValid())
	{
		UE_VLOG_UELOG(this, LogMassNavigation, Warning, TEXT("Trying to set lane state %s on an invalid lane %s\n"), *UEnum::GetValueAsString(NewState), *LaneHandle.ToString());
		return false;
	}
	
	FZoneGraphCrowdLaneData* CrowdLaneData = GetMutableCrowdLaneData(LaneHandle);
	const bool bSuccess = CrowdLaneData != nullptr;
	if (bSuccess)
	{
		CrowdLaneData->SetState(NewState);

		checkf(ZoneGraphAnnotationSubsystem != nullptr, TEXT("ZoneGraphAnnotationSubsystem should be initialized from the subsystem collection dependencies."));
		ZoneGraphAnnotationSubsystem->SendEvent(FZoneGraphCrowdLaneStateChangeEvent(LaneHandle, NewState));

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		DebugOnMassCrowdLaneStateChanged.Broadcast();
#endif
	}
	return bSuccess;
}

FCrowdTrackingLaneData& UMassCrowdSubsystem::CreateTrackingData(const int32 LaneIndex, const FZoneGraphStorage& ZoneGraphStorage)
{
	checkf(RegisteredLaneData.IsValidIndex(ZoneGraphStorage.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[ZoneGraphStorage.DataHandle.Index];
	return CrowdLaneData.LaneToTrackingDataLookup.Add(LaneIndex);
}

FCrowdBranchingLaneData& UMassCrowdSubsystem::CreateBranchingData(const int32 LaneIndex, const FZoneGraphStorage& ZoneGraphStorage)
{
	checkf(RegisteredLaneData.IsValidIndex(ZoneGraphStorage.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[ZoneGraphStorage.DataHandle.Index];
	FCrowdBranchingLaneData& NewWaitingData = CrowdLaneData.LaneToBranchingDataLookup.Add(LaneIndex);

	// Fetch density tag from upcoming lane
	// Keep default if unable to propagate a single density (no linked lanes or more than one outgoing lane)
	TArray<FZoneGraphLinkedLane> Links;
	UE::ZoneGraph::Query::GetLinkedLanes(ZoneGraphStorage, LaneIndex, EZoneLaneLinkType::Outgoing, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, Links);
	if (Links.Num() == 1)
	{
		const uint32 LinkLaneMask = ZoneGraphStorage.Lanes[Links[0].DestLane.Index].Tags.GetValue();
		NewWaitingData.DensityMask = LinkLaneMask & DensityMask.GetValue();
	}

	return NewWaitingData;
}

void UMassCrowdSubsystem::CreateWaitSlots(const int32 CrossingLaneIndex, FCrowdWaitAreaData& WaitArea, const FZoneGraphStorage& ZoneGraphStorage)
{
	checkf(RegisteredLaneData.IsValidIndex(ZoneGraphStorage.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));

	// Figure out total width of the crossing lane, including adjacent lanes.
	TArray<FZoneGraphLinkedLane> Links;
	UE::ZoneGraph::Query::GetLinkedLanes(ZoneGraphStorage, CrossingLaneIndex, EZoneLaneLinkType::Adjacent, EZoneLaneLinkFlags::Left | EZoneLaneLinkFlags::Right, EZoneLaneLinkFlags::None, Links);

	const FZoneLaneData& CrossingLane = ZoneGraphStorage.Lanes[CrossingLaneIndex];

	float SpaceLeft = CrossingLane.Width * 0.5f;
	float SpaceRight = CrossingLane.Width * 0.5f;
	
	for (const FZoneGraphLinkedLane& Link : Links)
	{
		const FZoneLaneData& AdjacentLane = ZoneGraphStorage.Lanes[Link.DestLane.Index];
		if (Link.HasFlags(EZoneLaneLinkFlags::Left))
		{
			SpaceLeft += AdjacentLane.Width;
		}
		else // Right
		{
			SpaceRight += AdjacentLane.Width;
		}
	}	

	const float TotalSpace = SpaceLeft + SpaceRight;

	// Distribute slots along the total width.
	checkf(MassCrowdSettings, TEXT("UMassCrowdSettings CDO should have been cached in Initialize"));
	const float SlotSize = (float)MassCrowdSettings->SlotSize;
	const int32 NumSlots = FMath::Max((SlotSize > 0 ? FMath::RoundToInt(TotalSpace / SlotSize) : 1), 1);

	const FVector Forward = ZoneGraphStorage.LaneTangentVectors[CrossingLane.PointsBegin];
	const FVector Up = ZoneGraphStorage.LaneUpVectors[CrossingLane.PointsBegin];
	const FVector Left = FVector::CrossProduct(Forward, Up);
	const FVector Base = ZoneGraphStorage.LanePoints[CrossingLane.PointsBegin] - Forward * MassCrowdSettings->SlotOffset;

	WaitArea.Slots.Reserve(NumSlots);
	for (int32 SlotIndex = 0; SlotIndex < NumSlots; SlotIndex++)
	{
		const float U = (SlotIndex + 0.5f) / (float)NumSlots;
		FCrowdWaitSlot& Slot = WaitArea.Slots.AddDefaulted_GetRef();
		Slot.Forward = Forward;
		// @todo This calculate is a bit kludge. The idea is to narrow the used space by zigzag pattern, so that there's enough space for incoming pedestrians to pass through. 
		Slot.Position = Base - Left * (SpaceRight + SlotSize/2 - TotalSpace * U * 0.75f) - Forward * ((SlotIndex & 1) * SlotSize*0.5f);
		Slot.Radius = SlotSize * 0.5f;
	}
	
	WaitArea.NumFreeSlots = NumSlots;
}

void UMassCrowdSubsystem::UpdateDensityMask()
{
	check(MassCrowdSettings);
	DensityMask = FZoneGraphTagMask::None;
	const TArray<FMassCrowdLaneDensityDesc>& Densities = MassCrowdSettings->GetLaneDensities();
	for (const FMassCrowdLaneDensityDesc& DensityDesc : Densities)
	{
		DensityMask.Add(DensityDesc.Tag);
	}
}

void UMassCrowdSubsystem::OnEnterTrackedLane(const FMassEntityHandle Entity, const int32 LaneIndex, FCrowdTrackingLaneData& TrackingData)
{
	++TrackingData.NumEntitiesOnLane;

	UE_VLOG_UELOG(this, LogMassNavigation, Verbose, TEXT("[%s] enters lane %d. Num entities on lane: %d"), *Entity.DebugGetDescription(), LaneIndex, TrackingData.NumEntitiesOnLane);
}

void UMassCrowdSubsystem::OnExitTrackedLane(const FMassEntityHandle Entity, const int32 LaneIndex, FCrowdTrackingLaneData& TrackingData)
{
	ensureMsgf(TrackingData.NumEntitiesOnLane >= 1, TEXT("OnExitTrackedLane should not be called more often than OnEnterTrackedLane"));
	--TrackingData.NumEntitiesOnLane;

	UE_VLOG_UELOG(this, LogMassNavigation, Verbose, TEXT("[%s] exits lane %d. Num entities on lane: %d"), *Entity.DebugGetDescription(), LaneIndex, TrackingData.NumEntitiesOnLane);
}

int32 UMassCrowdSubsystem::AcquireWaitingSlot(const FMassEntityHandle Entity, const FVector& EntityPosition, const FZoneGraphLaneHandle LaneHandle,
											  FVector& OutSlotPosition, FVector& OutSlotDirection)
{
	if (!LaneHandle.IsValid())
	{
		return INDEX_NONE;
	}
	
	checkf(RegisteredLaneData.IsValidIndex(LaneHandle.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[LaneHandle.DataHandle.Index];

	int32 BestSlotIndex = INDEX_NONE;
	FVector BestSlotPosition = FVector::ZeroVector;
	FVector BestSlotForward = FVector::ForwardVector;

	const FCrowdTrackingLaneData* TrackingData = CrowdLaneData.LaneToTrackingDataLookup.Find(LaneHandle.Index);
	if (TrackingData && TrackingData->WaitAreaIndex != INDEX_NONE)
	{
		FCrowdWaitAreaData& WaitArea = CrowdLaneData.WaitAreas[TrackingData->WaitAreaIndex];
		
		if (!WaitArea.IsFull())
		{
			check(WaitArea.NumFreeSlots > 0);
			
			// Find best vacant slot
			// The most distant slot is used so that later arrivals are less likely to need passing between already standing agents. 
			FVector::FReal BestDistanceSq = 0.;
			for (int32 SlotIndex = 0; SlotIndex < WaitArea.Slots.Num(); SlotIndex++)
			{
				const FCrowdWaitSlot& Slot = WaitArea.Slots[SlotIndex];
				if (!Slot.bOccupied)
				{
					const FVector::FReal DistanceToSlotSq = FVector::DistSquared(EntityPosition, Slot.Position);
					if (DistanceToSlotSq > BestDistanceSq)
					{
						BestDistanceSq = DistanceToSlotSq;
						BestSlotPosition = Slot.Position;
						BestSlotForward = Slot.Forward;
						BestSlotIndex = SlotIndex;
					}
				}
			}

			if (BestSlotIndex != INDEX_NONE)
			{
				WaitArea.Slots[BestSlotIndex].bOccupied = true;
				WaitArea.NumFreeSlots--;
			}
			
			// Signal if the lane became full.
			if (WaitArea.IsFull())
			{
				for (const FZoneGraphLaneHandle ConnectedLaneHandle : WaitArea.ConnectedLanes)
				{
					if (const FZoneGraphCrowdLaneData* LaneData = GetMutableCrowdLaneData(ConnectedLaneHandle))
					{
						checkf(ZoneGraphAnnotationSubsystem != nullptr, TEXT("ZoneGraphAnnotationSubsystem should be initialized from the subsystem collection dependencies."));
						// Resend the current state, to signal the ZoneGraphCrowdLaneAnnotations to update the tags.
						// The annotations is responsible for dealing with Waiting/Closed.
						// @todo: improve the logic when we dont have two systems using state differently.
						ZoneGraphAnnotationSubsystem->SendEvent(FZoneGraphCrowdLaneStateChangeEvent(ConnectedLaneHandle, LaneData->GetState()));
					}
				}
			}
		}
	}

	if (BestSlotIndex != INDEX_NONE)
	{
		OutSlotPosition = BestSlotPosition;
		OutSlotDirection = BestSlotForward;
	}
	
	return BestSlotIndex;
}

void UMassCrowdSubsystem::ReleaseWaitingSlot(const FMassEntityHandle Entity, const FZoneGraphLaneHandle LaneHandle, const int32 SlotIndex)
{
	if (!LaneHandle.IsValid())
	{
		return;
	}

	checkf(RegisteredLaneData.IsValidIndex(LaneHandle.DataHandle.Index), TEXT("Storage must have been allocated before creating lane data"));
	FRegisteredCrowdLaneData& CrowdLaneData = RegisteredLaneData[LaneHandle.DataHandle.Index];

	const FCrowdTrackingLaneData* TrackingData = CrowdLaneData.LaneToTrackingDataLookup.Find(LaneHandle.Index);

	if (TrackingData && TrackingData->WaitAreaIndex != INDEX_NONE)
	{
		FCrowdWaitAreaData& WaitArea = CrowdLaneData.WaitAreas[TrackingData->WaitAreaIndex];

		if (!WaitArea.Slots.IsValidIndex(SlotIndex))
		{
			UE_VLOG_UELOG(this, LogMassNavigation, Error, TEXT("%s Trying to release invalid slot index %d (max %d) on lane %d"),
				*Entity.DebugGetDescription(), SlotIndex, WaitArea.Slots.Num(), LaneHandle.Index);
			return;
		}

		const bool bWasFull = WaitArea.IsFull();

		if (WaitArea.Slots[SlotIndex].bOccupied == false)
		{
			UE_VLOG_UELOG(this, LogMassNavigation, Error, TEXT("%s Trying to release already released waiting slot %d on lane %d"),
				*Entity.DebugGetDescription(), SlotIndex, LaneHandle.Index);
		}
		else
		{
			WaitArea.Slots[SlotIndex].bOccupied = false;
			WaitArea.NumFreeSlots++;
			
			check(WaitArea.NumFreeSlots <= WaitArea.Slots.Num())
		}
		
		// Signal if the lane became vacant.
		if (bWasFull)
		{
			for (const FZoneGraphLaneHandle ConnectedLaneHandle : WaitArea.ConnectedLanes)
			{
				if (const FZoneGraphCrowdLaneData* LaneData = GetMutableCrowdLaneData(ConnectedLaneHandle))
				{
					checkf(ZoneGraphAnnotationSubsystem != nullptr, TEXT("ZoneGraphAnnotationSubsystem should be initialized from the subsystem collection dependencies."));
					// Resend the current state, to signal the ZoneGraphCrowdLaneAnnotations to update the tags.
					// The annotations is responsible for dealing with Waiting/Closed.
					// @todo: improve the logic when we dont have two systems using the state differently.
					ZoneGraphAnnotationSubsystem->SendEvent(FZoneGraphCrowdLaneStateChangeEvent(ConnectedLaneHandle, LaneData->GetState()));
				}
			}
		}
	}
}

