// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdVisualizationLODProcessor.h"
#include "MassCommonFragments.h"
#include "MassCommonTypes.h"
#include "MassExecutionContext.h"
#include "MassCrowdFragments.h"
#include "MassDebugger.h"
#include "MassEntityManager.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdVisualizationLODProcessor)

namespace UE::MassCrowd
{

	MASSCROWD_API int32 GCrowdTurnOffVisualization = 0;
	FAutoConsoleVariableRef CVarCrowdTurnOffVisualization(TEXT("Mass.CrowdTurnOffVisualization"), GCrowdTurnOffVisualization, TEXT("Turn off crowd visualization"));

	int32 bDebugCrowdVisualizationLOD = 0;
	int32 bDebugShowISMUnderSpecifiedRange = 0;

	FAutoConsoleVariableRef ConsoleVariables[] =
	{
		FAutoConsoleVariableRef(TEXT("mass.debug.crowd.VisualizationLOD"), bDebugCrowdVisualizationLOD, TEXT("Debug crowd visualization LOD"), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("mass.debug.crowd.ShowISMUnderSpecifiedRange"), bDebugShowISMUnderSpecifiedRange, TEXT("Show ISM under a specified range (meters)"), ECVF_Cheat)
	};

} // UE::MassCrowd

//----------------------------------------------------------------------//
// UMassCrowdVisualizationLODProcessor
//----------------------------------------------------------------------//
UMassCrowdVisualizationLODProcessor::UMassCrowdVisualizationLODProcessor()
{
	bAutoRegisterWithProcessingPhases = true;

	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::LOD;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);
}

void UMassCrowdVisualizationLODProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);

	CloseEntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	CloseEntityAdjustDistanceQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	FarEntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	DebugEntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);

	FilterTag = FMassCrowdTag::StaticStruct();
}

void UMassCrowdVisualizationLODProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	ForceOffLOD((bool)UE::MassCrowd::GCrowdTurnOffVisualization);

	TRACE_CPUPROFILER_EVENT_SCOPE(CrowdVisualizationLOD)

	Super::Execute(EntityManager, Context);

#if WITH_MASSGAMEPLAY_DEBUG
	if (UE::MassCrowd::bDebugCrowdVisualizationLOD)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayLOD)

		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			DebugEntityQuery.ForEachEntityChunk(Context, [DebugDrawingWorld](FMassExecutionContext& Context)
			{
				FMassVisualizationLODSharedFragment& LODSharedFragment = Context.GetMutableSharedFragment<FMassVisualizationLODSharedFragment>();
				const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
				const TConstArrayView<FMassRepresentationLODFragment> VisualizationLODList = Context.GetFragmentView<FMassRepresentationLODFragment>();
				LODSharedFragment.LODCalculator.DebugDisplayLOD(Context, VisualizationLODList, LocationList, DebugDrawingWorld);
			});
		}
	}
#endif // WITH_MASSGAMEPLAY_DEBUG

	if (UE::MassCrowd::bDebugShowISMUnderSpecifiedRange > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ShowISMUnderSpecifiedRange)
		if (UWorld* DebugDrawingWorld = UE::Mass::Debug::GetDebugDrawWorld(Context))
		{
			DebugEntityQuery.ForEachEntityChunk(Context, [DebugDrawingWorld](FMassExecutionContext& Context)
			{
				const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
				const TConstArrayView<FMassRepresentationFragment> RepresentationFragmentList = Context.GetFragmentView<FMassRepresentationFragment>();
				const TConstArrayView<FMassViewerInfoFragment> LODInfoFragmentList = Context.GetFragmentView<FMassViewerInfoFragment>();
				const float SpecifiedRangeSquaredCentimeters = static_cast<float>(FMath::Square(UE::MassCrowd::bDebugShowISMUnderSpecifiedRange * 100));
				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const FMassRepresentationFragment& RepresentationFragment = RepresentationFragmentList[EntityIt];
					const FMassViewerInfoFragment& LODInfoFragment = LODInfoFragmentList[EntityIt];
					if (UE::Mass::Representation::IsValidMeshRepresentation(RepresentationFragment.CurrentRepresentation) && SpecifiedRangeSquaredCentimeters > LODInfoFragment.ClosestViewerDistanceSq)
					{
						const FTransformFragment& EntityLocation = LocationList[EntityIt];
						DrawDebugSolidBox(DebugDrawingWorld, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 150.0f), FVector(50.0f), FColor::Red);
					}
			   }
		   });
		}
	}
}

//----------------------------------------------------------------------//
// UMassCrowdLODCollectorProcessor
//----------------------------------------------------------------------//
UMassCrowdLODCollectorProcessor::UMassCrowdLODCollectorProcessor()
{
	bAutoRegisterWithProcessingPhases = true;
}

void UMassCrowdLODCollectorProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);

	EntityQuery_VisibleRangeAndOnLOD.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery_VisibleRangeOnly.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery_OnLODOnly.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
	EntityQuery_NotVisibleRangeAndOffLOD.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
}
