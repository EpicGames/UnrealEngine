// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdVisualizationProcessor.h"
#include "MassCrowdFragments.h"
#include "MassExecutionContext.h"
#include "MassActorSubsystem.h"
#include "MassRepresentationFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdVisualizationProcessor)

namespace UE::MassCrowd
{
	int32 bDebugCrowdVisualType = 0;
	FAutoConsoleVariableRef CVarDebugVisualType(TEXT("mass.debug.CrowdVisualType"), bDebugCrowdVisualType, TEXT("Debug Crowd Visual Type"), ECVF_Cheat);
} // UE::MassCrowd

//----------------------------------------------------------------------//
// UMassCrowdVisualizationProcessor
//----------------------------------------------------------------------//
UMassCrowdVisualizationProcessor::UMassCrowdVisualizationProcessor()
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	bAutoRegisterWithProcessingPhases = true;

	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);

	bRequiresGameThreadExecution = true;
}

void UMassCrowdVisualizationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);
}

//----------------------------------------------------------------------//
// UMassDebugCrowdVisualizationProcessor
//----------------------------------------------------------------------//
UMassDebugCrowdVisualizationProcessor::UMassDebugCrowdVisualizationProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
	ExecutionOrder.ExecuteAfter.Add(UMassCrowdVisualizationProcessor::StaticClass()->GetFName());

	bRequiresGameThreadExecution = true;
}

void UMassDebugCrowdVisualizationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddTagRequirement<FMassCrowdTag>(EMassFragmentPresence::All);

	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.RequireMutatingWorldAccess(); // due to UWorld mutable access
}

void UMassDebugCrowdVisualizationProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	World = Owner.GetWorld();
	check(World);
}

void UMassDebugCrowdVisualizationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (UE::MassCrowd::bDebugCrowdVisualType)
	{
		check(World);

		TRACE_CPUPROFILER_EVENT_SCOPE(DebugDisplayVisualType)

		EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassRepresentationFragment> VisualizationList = Context.GetFragmentView<FMassRepresentationFragment>();
			const TConstArrayView<FMassActorFragment> ActorList = Context.GetFragmentView<FMassActorFragment>();
			const TConstArrayView<FTransformFragment> EntityLocationList = Context.GetFragmentView<FTransformFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FTransformFragment& EntityLocation = EntityLocationList[EntityIt];
				const FMassRepresentationFragment& Visualization = VisualizationList[EntityIt];
				const FMassActorFragment& ActorInfo = ActorList[EntityIt];
				const int32 RepresentationTypeIdx = (int32)Visualization.CurrentRepresentation;
				// Show replicated actors
				if (ActorInfo.IsValid() && !ActorInfo.IsOwnedByMass())
				{
					DrawDebugBox(World, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 120.0f), FVector(25.0f), UE::Mass::Representation::RepresentationTypeColors[RepresentationTypeIdx]);
				}
				else
				{ 
					DrawDebugSolidBox(World, EntityLocation.GetTransform().GetLocation() + FVector(0.0f, 0.0f, 120.0f), FVector(25.0f), UE::Mass::Representation::RepresentationTypeColors[RepresentationTypeIdx]);
				}
			}
		});
	}
}
