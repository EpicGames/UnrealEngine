// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCrowdVisualizationTrait.h"
#include "MassCrowdRepresentationSubsystem.h"
#include "MassCrowdRepresentationActorManagement.h"
#include "MassCrowdFragments.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdVisualizationTrait)

UMassCrowdVisualizationTrait::UMassCrowdVisualizationTrait()
{
	// Override the subsystem to support parallelization of the crowd
	RepresentationSubsystemClass = UMassCrowdRepresentationSubsystem::StaticClass();
	Params.RepresentationActorManagementClass = UMassCrowdRepresentationActorManagement::StaticClass();
	Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::HighResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::LowResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Low] = EMassRepresentationType::StaticMeshInstance;
	Params.LODRepresentation[EMassLOD::Off] = EMassRepresentationType::None;
	// Set bKeepLowResActor to true as a spawning optimization, this will keep the low-res actor if available while showing the static mesh instance
	Params.bKeepLowResActors = true;
	Params.bKeepActorExtraFrame = true;
	Params.bSpreadFirstVisualizationUpdate = false;
	Params.WorldPartitionGridNameContainingCollision = NAME_None;
	Params.NotVisibleUpdateRate = 0.5f;

	LODParams.BaseLODDistance[EMassLOD::High] = 0.f;
	LODParams.BaseLODDistance[EMassLOD::Medium] = 500.f;
	LODParams.BaseLODDistance[EMassLOD::Low] = 1000.f;
	LODParams.BaseLODDistance[EMassLOD::Off] = 5000.f;

	LODParams.VisibleLODDistance[EMassLOD::High] = 0.f;
	LODParams.VisibleLODDistance[EMassLOD::Medium] = 1000.f;
	LODParams.VisibleLODDistance[EMassLOD::Low] = 5000.f;
	LODParams.VisibleLODDistance[EMassLOD::Off] = 10000.f;

	LODParams.LODMaxCount[EMassLOD::High] = 10;
	LODParams.LODMaxCount[EMassLOD::Medium] = 20;
	LODParams.LODMaxCount[EMassLOD::Low] = 500;
	LODParams.LODMaxCount[EMassLOD::Off] = TNumericLimits<int32>::Max();

	LODParams.BufferHysteresisOnDistancePercentage = 20.0f;
	LODParams.DistanceToFrustum = 0.0f;
	LODParams.DistanceToFrustumHysteresis = 0.0f;

	LODParams.FilterTag = FMassCrowdTag::StaticStruct();
}

void UMassCrowdVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext,
	const UWorld& World) const
{
	Super::BuildTemplate(BuildContext, World);

	BuildContext.RequireTag<FMassCrowdTag>();
}
