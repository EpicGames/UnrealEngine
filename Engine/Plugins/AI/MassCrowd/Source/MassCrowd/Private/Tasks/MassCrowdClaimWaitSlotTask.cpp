// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassCrowdClaimWaitSlotTask.h"
#include "MassCrowdSubsystem.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassNavigationFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassCrowdClaimWaitSlotTask)

FMassCrowdClaimWaitSlotTask::FMassCrowdClaimWaitSlotTask()
{
	// This task should not react to Enter/ExitState when the state is reselected.
	bShouldStateChangeOnReselect = false;
}

bool FMassCrowdClaimWaitSlotTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(MoveTargetHandle);
	Linker.LinkExternalData(CrowdSubsystemHandle);

	return true;
}

void FMassCrowdClaimWaitSlotTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly<FMassZoneGraphLaneLocationFragment>();
	Builder.AddReadOnly<FMassMoveTargetFragment>();
	Builder.AddReadWrite(CrowdSubsystemHandle);
}

EStateTreeRunStatus FMassCrowdClaimWaitSlotTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	const FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FMassEntityHandle Entity = MassContext.GetEntity();
	
	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);
	UMassCrowdSubsystem& CrowdSubsystem = Context.GetExternalData(CrowdSubsystemHandle);

	FVector SlotPosition = FVector::ZeroVector;
	FVector SlotDirection = FVector::ForwardVector;
	InstanceData.WaitingSlotIndex = CrowdSubsystem.AcquireWaitingSlot(Entity, MoveTarget.Center, LaneLocation.LaneHandle, SlotPosition, SlotDirection);
	if (InstanceData.WaitingSlotIndex == INDEX_NONE)
	{
		// Failed to acquire slot
		return EStateTreeRunStatus::Failed;
	}
	
	InstanceData.AcquiredLane = LaneLocation.LaneHandle;

	InstanceData.WaitSlotLocation.LaneHandle = LaneLocation.LaneHandle;
	InstanceData.WaitSlotLocation.NextExitLinkType = EZoneLaneLinkType::None;
	InstanceData.WaitSlotLocation.NextLaneHandle.Reset();
	InstanceData.WaitSlotLocation.bMoveReverse = false;
	InstanceData.WaitSlotLocation.EndOfPathIntent = EMassMovementAction::Stand;
	InstanceData.WaitSlotLocation.EndOfPathPosition = SlotPosition;
	InstanceData.WaitSlotLocation.EndOfPathDirection = SlotDirection;
	InstanceData.WaitSlotLocation.TargetDistance = LaneLocation.LaneLength; // Go to end of lane
	// Let's start moving toward the interaction a bit before the entry point.
	InstanceData.WaitSlotLocation.AnticipationDistance.Set(100.f);
	
	return EStateTreeRunStatus::Running;
}

void FMassCrowdClaimWaitSlotTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FMassEntityHandle Entity = MassContext.GetEntity();

	UMassCrowdSubsystem& CrowdSubsystem = Context.GetExternalData(CrowdSubsystemHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	if (InstanceData.WaitingSlotIndex != INDEX_NONE)
	{
		CrowdSubsystem.ReleaseWaitingSlot(Entity, InstanceData.AcquiredLane, InstanceData.WaitingSlotIndex);
	}
	
	InstanceData.WaitingSlotIndex = INDEX_NONE;
	InstanceData.AcquiredLane.Reset();
	InstanceData.WaitSlotLocation.Reset();
}
