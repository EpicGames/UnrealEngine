// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassZoneGraphFindWanderTarget.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphQuery.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassAIBehaviorTypes.h"
#include "MassCrowdSettings.h"
#include "MassCrowdSubsystem.h"
#include "MassStateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "MassDebugger.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphFindWanderTarget)


FMassZoneGraphFindWanderTarget::FMassZoneGraphFindWanderTarget()
{
}

bool FMassZoneGraphFindWanderTarget::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(ZoneGraphSubsystemHandle);
	Linker.LinkExternalData(ZoneGraphAnnotationSubsystemHandle);
	Linker.LinkExternalData(MassCrowdSubsystemHandle);

	return true;
}

void FMassZoneGraphFindWanderTarget::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(LocationHandle);
	Builder.AddReadOnly(ZoneGraphSubsystemHandle);
	Builder.AddReadOnly(ZoneGraphAnnotationSubsystemHandle);
	Builder.AddReadOnly(MassCrowdSubsystemHandle);
}

EStateTreeRunStatus FMassZoneGraphFindWanderTarget::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);

	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetExternalData(ZoneGraphSubsystemHandle);
	const UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem = Context.GetExternalData(ZoneGraphAnnotationSubsystemHandle);
	const UMassCrowdSubsystem& MassCrowdSubsystem = Context.GetExternalData(MassCrowdSubsystemHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG

	if (!LaneLocation.LaneHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid lane location."));
		return EStateTreeRunStatus::Failed;
	}
			
	const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);
	if (!ZoneGraphStorage)
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Missing ZoneGraph Storage for current lane %s."), *LaneLocation.LaneHandle.ToString());
		return EStateTreeRunStatus::Failed;
	}

	const float MoveDistance = GetDefault<UMassCrowdSettings>()->GetMoveDistance();

	InstanceData.WanderTargetLocation.LaneHandle = LaneLocation.LaneHandle;
	InstanceData.WanderTargetLocation.TargetDistance = LaneLocation.DistanceAlongLane + MoveDistance;
	InstanceData.WanderTargetLocation.NextExitLinkType = EZoneLaneLinkType::None;
	InstanceData.WanderTargetLocation.NextLaneHandle.Reset();
	InstanceData.WanderTargetLocation.bMoveReverse = false;
	InstanceData.WanderTargetLocation.EndOfPathIntent = EMassMovementAction::Move;

	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Log, TEXT("Find wander target."));
	}

	EStateTreeRunStatus Status = EStateTreeRunStatus::Running;
	
	// When close to end of a lane, choose next lane.
	if (InstanceData.WanderTargetLocation.TargetDistance > LaneLocation.LaneLength)
	{
		InstanceData.WanderTargetLocation.TargetDistance = FMath::Min(InstanceData.WanderTargetLocation.TargetDistance, LaneLocation.LaneLength);

		typedef TTuple<const FZoneGraphLinkedLane, const float> FBranchingCandidate;
		TArray<FBranchingCandidate, TInlineAllocator<8>> Candidates;
		float CombinedWeight = 0.f;

		auto FindCandidates = [this, &ZoneGraphAnnotationSubsystem, &MassCrowdSubsystem, ZoneGraphStorage, LaneLocation, &Candidates, &CombinedWeight](const EZoneLaneLinkType Type)-> bool
		{
			TArray<FZoneGraphLinkedLane> LinkedLanes;
			UE::ZoneGraph::Query::GetLinkedLanes(*ZoneGraphStorage, LaneLocation.LaneHandle, Type, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, LinkedLanes);

			for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
			{
				// Apply tag filter
				const FZoneGraphTagMask BehaviorTags = ZoneGraphAnnotationSubsystem.GetAnnotationTags(LinkedLane.DestLane);
				if (!AllowedAnnotationTags.Pass(BehaviorTags))
				{
					continue;
				}

				// Add new candidate with its selection weight based on density
				const FZoneGraphTagMask& LaneTagMask = ZoneGraphStorage->Lanes[LinkedLane.DestLane.Index].Tags;
				const float Weight = MassCrowdSubsystem.GetDensityWeight(LinkedLane.DestLane, LaneTagMask);
				CombinedWeight += Weight;
				Candidates.Add(MakeTuple(LinkedLane, CombinedWeight));
			}

			return !Candidates.IsEmpty();
		};

		if (FindCandidates(EZoneLaneLinkType::Outgoing))
		{
			InstanceData.WanderTargetLocation.NextExitLinkType = EZoneLaneLinkType::Outgoing;
		}
		else
		{
			// Could not continue, try to switch to an adjacent lane.
			// @todo: we could try to do something smarter here so that agents do not clump up. May need to have some heuristic,
			//		  i.e. at intersections it looks better to switch lane immediately, with flee, it looks better to vary the location randomly.
			InstanceData.WanderTargetLocation.TargetDistance = LaneLocation.DistanceAlongLane;

			// Try adjacent lanes
			if (FindCandidates(EZoneLaneLinkType::Adjacent))
			{
				// Found adjacent lane, choose it once followed the short path. Keeping the random offset from above,
				// so that all agents dont follow until the end of the path to turn.
				InstanceData.WanderTargetLocation.NextExitLinkType = EZoneLaneLinkType::Adjacent;
			}
		}

		if (Candidates.IsEmpty())
		{
			// Could not find next lane, fail.
			InstanceData.WanderTargetLocation.Reset();
			Status = EStateTreeRunStatus::Failed;
		}
		else
		{
			// Select new lane based on the weight of each candidates
			const float Rand = FMath::RandRange(0.f, CombinedWeight);
			for (const FBranchingCandidate& Candidate : Candidates)
			{
				const float CandidateCombinedWeight = Candidate.Get<1>();
				if (Rand < CandidateCombinedWeight)
				{
					const FZoneGraphLinkedLane& LinkedLane = Candidate.Get<0>();
					InstanceData.WanderTargetLocation.NextLaneHandle = LinkedLane.DestLane;
					break;
				}
			}
		}
	}

	return Status;
}
