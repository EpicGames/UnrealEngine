// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphCrowdLaneAnnotations.h"
#include "Engine/Engine.h"
#include "MassCrowdSubsystem.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphRenderingUtilities.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSettings.h"
#include "MassNavigationTypes.h"
#include "VisualLogger/VisualLogger.h"

#if UE_ENABLE_DEBUG_DRAWING
#include "Engine/Canvas.h"
#endif // UE_ENABLE_DEBUG_DRAWING

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphCrowdLaneAnnotations)

void UZoneGraphCrowdLaneAnnotations::PostSubsystemsInitialized()
{
	Super::PostSubsystemsInitialized();

	CrowdSubsystem = UWorld::GetSubsystem<UMassCrowdSubsystem>(GetWorld());
	checkf(CrowdSubsystem, TEXT("Expecting MassCrowdSubsystem to be present."));
}

FZoneGraphTagMask UZoneGraphCrowdLaneAnnotations::GetAnnotationTags() const
{
	FZoneGraphTagMask AllTags;
	AllTags.Add(CloseLaneTag);
	AllTags.Add(WaitingLaneTag);

	return AllTags;
}

void UZoneGraphCrowdLaneAnnotations::HandleEvents(const FInstancedStructContainer& Events)
{
	for (FConstStructView Event : Events)
	{
		if (const FZoneGraphCrowdLaneStateChangeEvent* const StateChangeEvent = Event.GetPtr<const FZoneGraphCrowdLaneStateChangeEvent>())
		{
			StateChangeEvents.Add(*StateChangeEvent);
		}
	}
}

void UZoneGraphCrowdLaneAnnotations::TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& AnnotationTagContainer)
{
	if (!CloseLaneTag.IsValid())
	{
		return;
	}

	FZoneGraphTagMask AllTags;
	AllTags.Add(CloseLaneTag);
	AllTags.Add(WaitingLaneTag);

	// Process events
	for (const FZoneGraphCrowdLaneStateChangeEvent& Event : StateChangeEvents)
	{
		if (Event.Lane.IsValid())
		{
			TArrayView<FZoneGraphTagMask> LaneTags = AnnotationTagContainer.GetMutableAnnotationTagsForData(Event.Lane.DataHandle);
			FZoneGraphTagMask& LaneTagMask = LaneTags[Event.Lane.Index];

			LaneTagMask.Remove(AllTags);

			if (Event.State == ECrowdLaneState::Closed)
			{
				const FCrowdWaitAreaData* WaitArea = CrowdSubsystem->GetCrowdWaitingAreaData(Event.Lane);

				if (WaitArea && !WaitArea->IsFull())
				{
					LaneTagMask.Add(WaitingLaneTag);
				}
				else
				{
					LaneTagMask.Add(CloseLaneTag);
				}
			}
		}
		else
		{
			UE_VLOG_UELOG(this, LogMassNavigation, Warning, TEXT("Trying to set lane state %s on an invalid lane %s\n"), *UEnum::GetValueAsString(Event.State), *Event.Lane.ToString());
		}
	}
	StateChangeEvents.Reset();

#if UE_ENABLE_DEBUG_DRAWING
	if (bEnableDebugDrawing)
	{
		MarkRenderStateDirty();
	}
#endif // UE_ENABLE_DEBUG_DRAWING
}

#if UE_ENABLE_DEBUG_DRAWING
void UZoneGraphCrowdLaneAnnotations::DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy)
{
	UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	UZoneGraphAnnotationSubsystem* ZoneGraphAnnotationSubsystem = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(GetWorld());
	if (!ZoneGraph || !ZoneGraphAnnotationSubsystem)
	{
		return;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);

	static const FVector ZOffset(0, 0, 35.0f);
	static const FLinearColor WaitingColor(FColor(255, 196, 0));
	static const FLinearColor ClosedColor(FColor(255, 61, 0));

	FZoneGraphTagMask AllTags;
	AllTags.Add(CloseLaneTag);
	AllTags.Add(WaitingLaneTag);

	FVector ViewLocation = FVector::ZeroVector;
	FRotator ViewRotation = FRotator::ZeroRotator;
	GetFirstViewPoint(ViewLocation, ViewRotation);

	const float DrawDistance = GetMaxDebugDrawDistance();
	const float DrawDistanceSq = FMath::Square(DrawDistance);

	for (const FRegisteredCrowdLaneData& RegisteredLaneData : CrowdSubsystem->RegisteredLaneData)
	{
		const FZoneGraphStorage* ZoneStorage = ZoneGraph->GetZoneGraphStorage(RegisteredLaneData.DataHandle);
		if (ZoneStorage == nullptr)
		{
			continue;
		}

		for (const FZoneData& Zone : ZoneStorage->Zones)
		{
			const FVector::FReal DistanceSq = FVector::DistSquared(ViewLocation, Zone.Bounds.GetCenter());
			if (DistanceSq > DrawDistanceSq)
			{
				continue;
			}

			for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
			{
				const FZoneGraphCrowdLaneData& LaneData = RegisteredLaneData.CrowdLaneDataArray[LaneIdx];
				if (LaneData.GetState() == ECrowdLaneState::Closed)
				{
					const FZoneGraphLaneHandle LaneHandle(LaneIdx, RegisteredLaneData.DataHandle);

					FLinearColor Color = ClosedColor;

					const FCrowdWaitAreaData* WaitArea = CrowdSubsystem->GetCrowdWaitingAreaData(LaneHandle);
					if (WaitArea && !WaitArea->IsFull())
					{
						Color = WaitingColor;
					}

					UE::ZoneGraph::RenderingUtilities::AppendLane(DebugProxy, *ZoneStorage, LaneHandle, Color.ToFColor(/*sRGB*/true), 4.0f, ZOffset);
				}
			}
		}
		
		auto AppendCircleXY = [DebugProxy](const FVector& Center, const float Radius, const FColor Color, const float LineThickness)
		{
			static int32 NumDivs = 16;

			FVector PrevPoint;
			for (int32 Index = 0; Index <= NumDivs; Index++)
			{
				const float Angle = (float)Index / (float)NumDivs * PI * 2.0f;
				float DirX, DirY;
				FMath::SinCos(&DirX, &DirY, Angle);
				const FVector Dir(DirX, DirY, 0.0f);
				const FVector Point = Center + Dir * Radius;
				if (Index > 0)
				{
					DebugProxy->Lines.Emplace(PrevPoint, Point, Color, LineThickness);
				}
				PrevPoint = Point;
			}
		};

		const FColor SlotColor = FColor::Orange;
		for (const FCrowdWaitAreaData& WaitArea : RegisteredLaneData.WaitAreas)
		{
			if (WaitArea.Slots.Num() > 0)
			{
				const FVector::FReal DistanceSq = FVector::DistSquared(ViewLocation, WaitArea.Slots[0].Position);
				if (DistanceSq < DrawDistanceSq)
				{
					for (const FCrowdWaitSlot& Slot : WaitArea.Slots)
					{
						AppendCircleXY(Slot.Position + ZOffset, Slot.Radius, SlotColor, 1.0f);
						DebugProxy->Lines.Emplace(Slot.Position + ZOffset, Slot.Position + Slot.Forward * Slot.Radius + ZOffset, SlotColor, 4.0f);
					}
				}
			}
		}
	}
}

void UZoneGraphCrowdLaneAnnotations::DebugDrawCanvas(UCanvas* Canvas, APlayerController*)
{
	if (!bEnableDebugDrawing)
	{
		return;
	}

	const FColor OldDrawColor = Canvas->DrawColor;
	const UFont* RenderFont = GEngine->GetSmallFont();

	const FFontRenderInfo FontInfo = Canvas->CreateFontRenderInfo(/*bClipText*/true, /*bEnableShadow*/true);

	Canvas->SetDrawColor(FColor::White);
	static const FVector ZOffset(0, 0, 35.0f);

	UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	UZoneGraphAnnotationSubsystem* ZoneGraphAnnotationSubsystem = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(GetWorld());
	if (!ZoneGraph)
	{
		return;
	}

	if (Canvas->SceneView == nullptr)
	{
		return;
	}
	
	const FVector ViewLocation = Canvas->SceneView->ViewLocation;
	const float DrawDistance = GetMaxDebugDrawDistance() * 0.25f;
	const FVector::FReal DrawDistanceSq = FMath::Square(DrawDistance);

	auto InFrustum = [Canvas](const FVector& Location)
	{
		return Canvas->SceneView->ViewFrustum.IntersectBox(Location, FVector::ZeroVector);
	};

	for (const FRegisteredCrowdLaneData& RegisteredLaneData : CrowdSubsystem->RegisteredLaneData)
	{
		const FZoneGraphStorage* ZoneStorage = RegisteredLaneData.DataHandle.IsValid() ? ZoneGraph->GetZoneGraphStorage(RegisteredLaneData.DataHandle) : nullptr;
		if (ZoneStorage == nullptr)
		{
			continue;
		}

		for (const FZoneData& Zone : ZoneStorage->Zones)
		{
			const FVector::FReal DistanceSq = FVector::DistSquared(ViewLocation, Zone.Bounds.GetCenter());
			if (DistanceSq > DrawDistanceSq)
			{
				continue;
			}

			for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
			{
				const FZoneGraphLaneHandle LaneHandle(LaneIdx, RegisteredLaneData.DataHandle);
				
				FZoneGraphLaneLocation CenterLoc;
				UE::ZoneGraph::Query::CalculateLocationAlongLaneFromRatio(*ZoneStorage, LaneIdx, 0.5f, CenterLoc);

				if (!InFrustum(CenterLoc.Position))
				{
					continue;
				}
				
				const FVector ScreenLoc = Canvas->Project(CenterLoc.Position, /*bClampToNearPlane*/false);
				
				// Flags
				if (bDisplayTags)
				{
					const FZoneGraphTagMask Mask = ZoneGraphAnnotationSubsystem->GetAnnotationTags(LaneHandle);
					Canvas->DrawText(RenderFont, FString::Printf(TEXT("%s\n0x%08X"), *UE::ZoneGraph::Helpers::GetTagMaskString(Mask, TEXT(", ")), Mask.GetValue())
						, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
						, FloatCastChecked<float>(ScreenLoc.Y, UE::LWC::DefaultFloatPrecision)
						, /*XScale*/1.0f, /*YScale*/1.0f, FontInfo);
				}

				// Tracking
				if (const FCrowdTrackingLaneData* TrackingData = RegisteredLaneData.LaneToTrackingDataLookup.Find(LaneIdx))
				{
					if (TrackingData->NumEntitiesOnLane > 0)
					{
						Canvas->DrawText(RenderFont, FString::Printf(TEXT("Num Entities: %d"), TrackingData->NumEntitiesOnLane)
						, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
						, FloatCastChecked<float>(ScreenLoc.Y + 20, UE::LWC::DefaultFloatPrecision)
						, /*XScale*/1.0f, /*YScale*/1.0f, FontInfo);
					}
				}
			}
		}

		// Waiting areas
		for (const FCrowdWaitAreaData& WaitArea : RegisteredLaneData.WaitAreas)
		{
			if (WaitArea.Slots.Num() > 0)
			{
				const FVector::FReal DistanceSq = FVector::DistSquared(ViewLocation, WaitArea.Slots[0].Position);
				if (DistanceSq < DrawDistanceSq)
				{
					for (const FCrowdWaitSlot& Slot : WaitArea.Slots)
					{
						if (Slot.bOccupied)
						{
							if (!InFrustum(Slot.Position + ZOffset))
							{
								continue;
							}

							const FVector ScreenLoc = Canvas->Project(Slot.Position + ZOffset);
							Canvas->DrawText(RenderFont, TEXT("OCCUPIED")
								, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
								, FloatCastChecked<float>(ScreenLoc.Y, UE::LWC::DefaultFloatPrecision)
								, /*XScale*/1.0f, /*YScale*/1.0f, FontInfo);
						}
					}
				}
			}
		}
	}

	Canvas->SetDrawColor(OldDrawColor);
}

#endif // UE_ENABLE_DEBUG_DRAWING
