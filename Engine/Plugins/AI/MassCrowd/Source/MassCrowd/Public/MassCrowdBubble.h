// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationPathHandlers.h"
#include "MassReplicationTransformHandlers.h"
#include "MassCrowdReplicatedAgent.h"
#include "MassClientBubbleHandler.h"
#include "MassClientBubbleInfoBase.h"
#include "MassEntityView.h"
#include "MassCrowdBubble.generated.h"

#define UE_API MASSCROWD_API

class FMassCrowdClientBubbleHandler;

class FMassCrowdClientBubbleHandler : public TClientBubbleHandlerBase<FCrowdFastArrayItem>
{
public:
	typedef TClientBubbleHandlerBase<FCrowdFastArrayItem> Super;
	typedef TMassClientBubblePathHandler<FCrowdFastArrayItem> FMassClientBubblePathHandler;
	typedef TMassClientBubbleTransformHandler<FCrowdFastArrayItem> FMassClientBubbleTransformHandler;

	FMassCrowdClientBubbleHandler()
		: PathHandler(*this)
		, TransformHandler(*this)
	{}

#if UE_REPLICATION_COMPILE_SERVER_CODE
	const FMassClientBubblePathHandler& GetPathHandler() const { return PathHandler; }
	FMassClientBubblePathHandler& GetPathHandlerMutable() { return PathHandler; }

	const FMassClientBubbleTransformHandler& GetTransformHandler() const { return TransformHandler; }
	FMassClientBubbleTransformHandler& GetTransformHandlerMutable() { return TransformHandler; }
#endif // UE_REPLICATION_COMPILE_SERVER_CODE


protected:
#if UE_REPLICATION_COMPILE_CLIENT_CODE
	UE_API virtual void PostReplicatedAdd(const TArrayView<int32> AddedIndices, int32 FinalSize) override;
	UE_API virtual void PostReplicatedChange(const TArrayView<int32> ChangedIndices, int32 FinalSize) override;

	UE_API void PostReplicatedChangeEntity(const FMassEntityView& EntityView, const FReplicatedCrowdAgent& Item) const;
#endif //UE_REPLICATION_COMPILE_CLIENT_CODE

#if WITH_MASSGAMEPLAY_DEBUG && WITH_EDITOR
	UE_API virtual void DebugValidateBubbleOnServer() override;
	UE_API virtual void DebugValidateBubbleOnClient() override;
#endif // WITH_MASSGAMEPLAY_DEBUG

	FMassClientBubblePathHandler PathHandler;
	FMassClientBubbleTransformHandler TransformHandler;
};

/** Mass client bubble, there will be one of these per client and it will handle replicating the fast array of Agents between the server and clients */
USTRUCT()
struct FMassCrowdClientBubbleSerializer : public FMassClientBubbleSerializerBase
{
	GENERATED_BODY()

	FMassCrowdClientBubbleSerializer()
	{
		Bubble.Initialize(Crowd, *this);
	};
		
	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParams)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FCrowdFastArrayItem, FMassCrowdClientBubbleSerializer>(Crowd, DeltaParams, *this);
	}

public:
	FMassCrowdClientBubbleHandler Bubble;

protected:
	/** Fast Array of Agents for efficient replication. Maintained as a freelist on the server, to keep index consistency as indexes are used as Handles into the Array 
	 *  Note array order is not guaranteed between server and client so handles will not be consistent between them, FMassNetworkID will be.
	 */
	UPROPERTY(Transient)
	TArray<FCrowdFastArrayItem> Crowd;
};

template<>
struct TStructOpsTypeTraits<FMassCrowdClientBubbleSerializer> : public TStructOpsTypeTraitsBase2<FMassCrowdClientBubbleSerializer>
{
	enum
	{
		WithNetDeltaSerializer = true,
		WithCopy = false,
	};
};

/**
 *  This class will allow us to replicate Mass data based on the fidelity required for each player controller. There is one AMassReplicationActor per PlayerController and 
 *  which is also its owner.
 */
UCLASS(MinimalAPI)
class AMassCrowdClientBubbleInfo : public AMassClientBubbleInfoBase
{
	GENERATED_BODY()

public:
	UE_API AMassCrowdClientBubbleInfo(const FObjectInitializer& ObjectInitializer);

	FMassCrowdClientBubbleSerializer& GetCrowdSerializer() { return CrowdSerializer; }

protected:
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

protected:
	UPROPERTY(Replicated, Transient)
	FMassCrowdClientBubbleSerializer CrowdSerializer;
};

#undef UE_API
