// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphAnnotationTypes.h"
#include "MassCrowdFragments.generated.h"

/**
 * Special tag to differentiate the crowd from the rest of the other entities
 * Should not contain any data, this is purely a tag
 */
USTRUCT()
struct FMassCrowdTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Data fragment to store the last lane the agent was tracked on.
 */
USTRUCT()
struct FMassCrowdLaneTrackingFragment : public FMassFragment
{
	GENERATED_BODY()
	FZoneGraphLaneHandle TrackedLaneHandle;
};


USTRUCT()
struct FMassCrowdObstacleFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Obstacle ID reported to the obstruction annotation. */
	FMassLaneObstacleID LaneObstacleID;

	/** Position of the obstacle when it last moved. */
	FVector LastPosition = FVector::ZeroVector;

	/** Time since the obstacle has not moved based on speed tolerance. */
	float TimeSinceStopped = 0.0f;

	/** True of the current obstacle state is moving. */
	bool bIsMoving = true;
};
