// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZoneGraphTestingActor.h"
#include "MassCrowdSubsystem.h"
#include "MassCrowdLaneClosingTest.generated.h"

class UMassCrowdSubsystem;

UCLASS()
class UZoneGraphCloseCrowdLaneTest : public UZoneLaneTest
{
	GENERATED_BODY()

protected:
	virtual void OnLaneLocationUpdated(const FZoneGraphLaneLocation& PrevLaneLocation, const FZoneGraphLaneLocation& NextLaneLocation) override;
	virtual void Draw(FPrimitiveDrawInterface* PDI) const override;
	virtual void OnOwnerSet() override;

private:
	UPROPERTY(Transient)
	FZoneGraphLaneLocation LaneLocation;

	UPROPERTY(Transient)
	TObjectPtr<UMassCrowdSubsystem> CrowdSubsystem;

	UPROPERTY(EditAnywhere, Category = Test)
	ECrowdLaneState LaneState = ECrowdLaneState::Closed;

	UPROPERTY(VisibleAnywhere, Transient, Category = Test)
	ECrowdLaneState PrevLaneState = ECrowdLaneState::Opened;
};