// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "MassCrowdLaneDataRenderingComponent.generated.h"

#define UE_API MASSCROWD_API

/**
 * Primitive component that can be used to render runtime state of zone graph lanes (e.g. Opened|Closed, Density, etc.)
 * The component must be added on a ZoneGraphData actor.
 */
UCLASS(MinimalAPI, editinlinenew, meta = (BlueprintSpawnableComponent), hidecategories = (Object, LOD, Lighting, VirtualTexture, Transform, HLOD, Collision, TextureStreaming, Mobile, Physics, Tags, AssetUserData, Activation, Cooking, Rendering, Navigation))
class UMassCrowdLaneDataRenderingComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	UMassCrowdLaneDataRenderingComponent() = default;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
private:
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API void DebugDrawOnCanvas(UCanvas* Canvas, APlayerController*) const;
	FDelegateHandle DebugTextDrawingDelegateHandle;
	FDelegateHandle OnLaneStateChangedDelegateHandle;
#if WITH_EDITOR
	FDelegateHandle OnLaneRenderSettingsChangedDelegateHandle;
#endif // WITH_EDITOR
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST
};

#undef UE_API
