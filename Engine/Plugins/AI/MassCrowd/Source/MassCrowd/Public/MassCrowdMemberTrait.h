// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MassCrowdMemberTrait.generated.h"

#define UE_API MASSCROWD_API

/**
 * Trait to mark an entity with the crowd tag and add required fragments to track current lane
 */
UCLASS(MinimalAPI, meta = (DisplayName = "CrowdMember"))
class UMassCrowdMemberTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
