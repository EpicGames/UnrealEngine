// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSignalProcessorBase.h"
#include "MassObserverProcessor.h"
#include "MassCrowdFragments.h"
#include "MassCrowdNavigationProcessor.generated.h"

#define UE_API MASSCROWD_API

class UZoneGraphAnnotationSubsystem;
class UMassCrowdSubsystem;

/** Processor that monitors when entities change lane and notify the MassCrowd subsystem. */
UCLASS(MinimalAPI)
class UMassCrowdLaneTrackingSignalProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()
public:
	UE_API UMassCrowdLaneTrackingSignalProcessor();

protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& /*Unused*/) override;
};

/** Processors that cleans up the lane tracking on entity destruction. */
UCLASS(MinimalAPI)
class UMassCrowdLaneTrackingDestructor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassCrowdLaneTrackingDestructor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};


UCLASS(MinimalAPI)
class UMassCrowdDynamicObstacleProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassCrowdDynamicObstacleProcessor();

protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

	UE_API void OnStop(FMassCrowdObstacleFragment& OutObstacle, const float Radius);
	UE_API void OnMove(FMassCrowdObstacleFragment& OutObstacle);

	FMassEntityQuery EntityQuery_Conditional;

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphAnnotationSubsystem> ZoneGraphAnnotationSubsystem;
};


UCLASS(MinimalAPI)
class UMassCrowdDynamicObstacleInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassCrowdDynamicObstacleInitializer();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};


UCLASS(MinimalAPI)
class UMassCrowdDynamicObstacleDeinitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassCrowdDynamicObstacleDeinitializer();

protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphAnnotationSubsystem> ZoneGraphAnnotationSubsystem;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
