// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationTransformHandlers.h"
#include "MassReplicationTypes.h"
#include "MassClientBubbleHandler.h"
#include "MassReplicationPathHandlers.h"
#include "MassCrowdReplicatedAgent.generated.h"

/** The data that is replicated specific to each Crowd agent */
USTRUCT()
struct FReplicatedCrowdAgent : public FReplicatedAgentBase
{
	GENERATED_BODY()

	const FReplicatedAgentPathData& GetReplicatedPathData() const { return Path; }

	/** This function is required to be provided in FReplicatedAgentBase derived classes that use FReplicatedAgentPathData */
	FReplicatedAgentPathData& GetReplicatedPathDataMutable() { return Path; }

	const FReplicatedAgentPositionYawData& GetReplicatedPositionYawData() const { return PositionYaw; }

	/** This function is required to be provided in FReplicatedAgentBase derived classes that use FReplicatedAgentPositionYawData */
	FReplicatedAgentPositionYawData& GetReplicatedPositionYawDataMutable() { return PositionYaw; }

private:
	UPROPERTY(Transient)
	FReplicatedAgentPathData Path;

	UPROPERTY(Transient)
	FReplicatedAgentPositionYawData PositionYaw;
};

/** Fast array item for efficient agent replication. Remember to make this dirty if any FReplicatedCrowdAgent member variables are modified */
USTRUCT()
struct FCrowdFastArrayItem : public FMassFastArrayItemBase
{
	GENERATED_BODY()

	FCrowdFastArrayItem() = default;
	FCrowdFastArrayItem(const FReplicatedCrowdAgent& InAgent, const FMassReplicatedAgentHandle InHandle)
		: FMassFastArrayItemBase(InHandle)
		, Agent(InAgent)
	{}

	/** This typedef is required to be provided in FMassFastArrayItemBase derived classes (with the associated FReplicatedAgentBase derived class) */
	typedef FReplicatedCrowdAgent FReplicatedAgentType;

	UPROPERTY()
	FReplicatedCrowdAgent Agent;
};
