// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationProcessor.h"
#include "MassCrowdReplicator.generated.h"

#define UE_API MASSCROWD_API

/** Class that handles replication and only runs on the server. It queries Mass entity fragments and sets those values when appropriate using the MassClientBubbleHandler. */
UCLASS(MinimalAPI)
class UMassCrowdReplicator : public UMassReplicatorBase
{
	GENERATED_BODY()

public:
	/**
	 * Overridden to add specific entity query requirements for replication.
	 * Usually we add replication processor handler requirements.
	 */
	UE_API virtual void AddRequirements(FMassEntityQuery& EntityQuery) override;
	
	/**
	 * Overridden to process the client replication.
	 * This methods should call CalculateClientReplication with the appropriate callback implementation.
	 */
	UE_API virtual void ProcessClientReplication(FMassExecutionContext& Context, FMassReplicationContext& ReplicationContext) override;
};

#undef UE_API
