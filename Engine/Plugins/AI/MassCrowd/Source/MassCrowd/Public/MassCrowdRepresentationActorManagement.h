// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationActorManagement.h"
#include "MassCrowdRepresentationActorManagement.generated.h"

#define UE_API MASSCROWD_API

/**
 * Overridden representation processor to make it tied to the crowd via the requirements.
 * It is also the base class for all the different type of crowd representation (Visualization & ServerSideRepresentation)
 */
UCLASS(MinimalAPI, abstract)
class UMassCrowdRepresentationActorManagement : public UMassRepresentationActorManagement
{
	GENERATED_BODY()

protected:

	/**
	 * Enable/disable a spawned actor
	 * @param EnabledType is the type of enabling to do on this actor
	 * @param Actor is the actual actor to perform enabling type on
	 * @param EntityIdx is the entity index currently processing
	 * @param Context is the current Mass execution context
	 */
	UE_API virtual void SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx, FMassCommandBuffer& CommandBuffer) const override;

	/**
	 * Returns an actor of the template type and setup fragments values from it
	 * @param RepresentationSubsystem to use to get or spawn the actor
	 * @param EntitySubsystem associated to the mass agent
	 * @param MassAgent is the handle to the associated mass agent
	 * @param ActorInfo is the fragment where we are going to store the actor pointer
	 * @param Transform is the spatial information about where to spawn the actor
	 * @param TemplateActorIndex is the index of the type fetched with UMassRepresentationSubsystem::FindOrAddTemplateActor()
	 * @param SpawnRequestHandle (in/out) In: previously requested spawn Out: newly requested spawn
	 * @param Priority of this spawn request in comparison with the others, lower value means higher priority
	 * @return the actor spawned
	 */
	UE_API virtual AActor* GetOrSpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem, FMassEntityManager& EntityManager
		, const FMassEntityHandle MassAgent, const FTransform& Transform, const int16 TemplateActorIndex
		, FMassActorSpawnRequestHandle& SpawnRequestHandle, const float Priority) const override;

	/**
	 * Teleports the actor at the specified transform by preserving its velocity and without collision.
	 * The destination will be adjusted to fit an existing capsule.
	 * @param Transform is the new actor's transform 
	 * @param Actor is the actual actor to teleport
	 * @param Context is the current Mass execution context
	 */
	UE_API virtual void TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const override;
};

#undef UE_API
