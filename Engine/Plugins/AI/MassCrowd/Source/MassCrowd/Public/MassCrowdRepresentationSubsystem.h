// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationSubsystem.h"
#include "MassCrowdRepresentationSubsystem.generated.h"

#define UE_API MASSCROWD_API

/**
 * Subsystem responsible for all visual of mass crowd agents, will handle actors spawning and static mesh instances
 */
UCLASS(MinimalAPI)
class UMassCrowdRepresentationSubsystem : public UMassRepresentationSubsystem
{
	GENERATED_BODY()

protected:
	// USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// USubsystem END
};

#undef UE_API
