// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCrowdTypes.h"
#include "MassCrowdFragments.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassProcessor.h"
#include "MassLODCalculator.h"
#include "MassLODTickRateController.h"

#include "MassCrowdServerRepresentationLODProcessor.generated.h"

UCLASS(MinimalAPI, meta=(DisplayName="Crowd Simulation LOD"))
class UMassCrowdServerRepresentationLODProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	MASSCROWD_API UMassCrowdServerRepresentationLODProcessor();

protected:

	MASSCROWD_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCROWD_API virtual void InitializeInternal(UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSCROWD_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** Distance where each LOD becomes relevant */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	float LODDistance[EMassLOD::Max];

	/** Hysteresis percentage on delta between the LOD distances */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", meta = (ClampMin = "0.0", UIMin = "0.0"), config)
	float BufferHysteresisOnDistancePercentage = 10.0f;

	/** Maximum limit of entity per LOD */
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	int32 LODMaxCount[EMassLOD::Max];

	TMassLODCalculator<FLODDefaultLogic> LODCalculator;

	FMassEntityQuery EntityQuery;
};
