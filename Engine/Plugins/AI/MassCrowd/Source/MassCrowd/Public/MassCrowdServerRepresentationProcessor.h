// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationProcessor.h"
#include "MassCrowdServerRepresentationProcessor.generated.h"

#define UE_API MASSCROWD_API

/**
 * Overridden representation processor to make it tied to the crowd on the server via the requirements
 * It is the counter part of the crowd visualization processor on the client.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Mass Crowd Server Representation"))
class UMassCrowdServerRepresentationProcessor : public UMassRepresentationProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassCrowdServerRepresentationProcessor();

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

#undef UE_API
