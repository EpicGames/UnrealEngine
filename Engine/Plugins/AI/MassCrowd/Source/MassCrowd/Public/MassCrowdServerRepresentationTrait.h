// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRepresentationTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRepresentationFragments.h"
#include "GameFramework/Actor.h"
#include "MassCrowdServerRepresentationTrait.generated.h"


UCLASS(MinimalAPI, meta=(DisplayName="Crowd Server Representation"))
class UMassCrowdServerRepresentationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

	MASSCROWD_API UMassCrowdServerRepresentationTrait();

protected:

	MASSCROWD_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	/** Actor class of this agent when spawned on server */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	TSubclassOf<AActor> TemplateActor;

	/** Configuration parameters for the representation processor */
	UPROPERTY(EditAnywhere, Category = "Mass|Visual")
	FMassRepresentationParameters Params;
};
