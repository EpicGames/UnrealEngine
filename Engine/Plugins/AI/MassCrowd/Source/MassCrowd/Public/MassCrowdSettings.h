// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "ZoneGraphTypes.h"
#include "MassCrowdSettings.generated.h"

#define UE_API MASSCROWD_API

#if WITH_EDITOR
/** Called when density settings change. */
DECLARE_MULTICAST_DELEGATE(FOnMassCrowdLaneDataSettingsChanged);
/** Called when rendering settings change. */
DECLARE_MULTICAST_DELEGATE(FOnMassCrowdLaneRenderSettingsChanged);
#endif

/**
 * Structure holding data to associate lane densities to
 * weights so lane selection at intersection could use that
 * to maintain overall density during the simulation
 */
USTRUCT()
struct FMassCrowdLaneDensityDesc
{
	GENERATED_BODY()

	/** Default weight of a lane if it has no density tag */
	static constexpr float DefaultWeight = 1.0f;

	/** Tag representing the lane density. */
	UPROPERTY(EditAnywhere, Category = "Lane Density")
	FZoneGraphTag Tag;

	/**
	 * Weight associated to the current lane density.
	 * This weight is used during lane selection at intersection
	 * and the random selection will consider the weight of each
	 * lane and the combined weight of all lanes.
	 */
	UPROPERTY(EditAnywhere, Category = "Lane Density")
	float Weight = DefaultWeight;

	UPROPERTY(EditAnywhere, Category = "Lane Density", meta = (HideAlphaChannel))
	FColor RenderColor = FColor::Silver;
};

/**
 * Settings for the MassCrowd plugin.
 */
UCLASS(MinimalAPI, config = Plugins, defaultconfig, DisplayName = "Mass Crowd", dontCollapseCategories)
class UMassCrowdSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	const TArray<FMassCrowdLaneDensityDesc>& GetLaneDensities() const { return LaneDensities; }
	float GetMoveDistance() const
	{ 
		return FMath::Max(0.0f, MoveDistance + FMath::RandRange(-MoveDistanceRandomDeviation, MoveDistanceRandomDeviation));
	}

#if WITH_EDITOR
	mutable FOnMassCrowdLaneDataSettingsChanged OnMassCrowdLaneDataSettingsChanged;
	mutable FOnMassCrowdLaneRenderSettingsChanged OnMassCrowdLaneRenderSettingsChanged;
#endif

	/** Base thickness used to render lane data specific to crowd. */
	UPROPERTY(EditAnywhere, config, meta = (ClampMin = "0.0", UIMin = "0.0"), Category = "Lane Debug Rendering")
	float LaneBaseLineThickness = 5.0f;

	/** Z offset used to render lane data specific to crowd over the actual zone graph. */
	UPROPERTY(EditAnywhere, config, meta = (ClampMin = "0.0", UIMin = "0.0"), Category = "Lane Debug Rendering")
	float LaneRenderZOffset = 50.0f;

	/** Scale factor applied on the base thickness to render intersection lanes data. */
	UPROPERTY(EditAnywhere, config, meta = (ClampMin = "1.0", UIMin = "1.0", ClampMax = "10.0", UIMax = "10.0"), Category = "Lane Debug Rendering")
	float IntersectionLaneScaleFactor = 1.5f;

	/** Scale factor applied on the base or intersection thickness to render density outline. */
	UPROPERTY(EditAnywhere, config, meta = (ClampMin = "1.0", UIMin = "1.0", ClampMax = "10.0", UIMax = "10.0"), Category = "Lane Debug Rendering")
	float LaneDensityScaleFactor = 1.5f;

	/** Color used to render crowd lane that are opened for navigation. */
	UPROPERTY(EditAnywhere, config, Category = "Lane Density", meta = (HideAlphaChannel))
	FColor OpenedLaneColor = FColor::Green;

	/** Color used to render crowd lane that are closed to navigation. */
	UPROPERTY(EditAnywhere, config, Category = "Lane Density", meta = (HideAlphaChannel))
	FColor ClosedLaneColor = FColor::Red;

	/** Tag required on a lane to build crowd related runtime data for it and render it. */
	UPROPERTY(EditDefaultsOnly, config, Category = Lane)
	FZoneGraphTag CrowdTag;

	/** Tag required on a lane to build intersection crossing runtime data for it. */
	UPROPERTY(EditDefaultsOnly, config, Category = Lane)
	FZoneGraphTag CrossingTag;

	/** Distance reserved for each entity while waiting on an intersection lane. */
	UPROPERTY(EditDefaultsOnly, config, Category = WaitArea)
	uint32 SlotSize = 50;

	/** Offset from the lane entry where the slots are created. */
	UPROPERTY(EditDefaultsOnly, config, Category = WaitArea)
	float SlotOffset = 75;

	/** Text will be added on lanes with entity tracking or waiting area to indicate the current occupation */
	UPROPERTY(EditAnywhere, config, Category = Options)
	bool bDisplayTrackingData = true;

	/** Lanes will be displayed to indicates the current state. See MassCrowd settings for parameters. */
	UPROPERTY(EditAnywhere, config, Category = Options)
	bool bDisplayStates = true;

	/** Lanes will be displayed to represent their assigned densities. See MassCrowd settings for parameters. */
	UPROPERTY(EditAnywhere, config, Category = Options)
	bool bDisplayDensities = true;

	/** An obstacle is considered being stopped when it's speed is less than the tolerance. */
	UPROPERTY(EditAnywhere, config, Category = Obstacles, meta = (ClampMin = "0.0", UIMin = "0.0", ForceUnits = "cm/s"))
	float ObstacleStoppingSpeedTolerance = 5.0f;

	/** An obstacle is considered moving when it has moved this much after being stationary. */
	UPROPERTY(EditAnywhere, config, Category = Obstacles, meta = (ClampMin = "0.0", UIMin = "0.0", ForceUnits = "cm"))
	float ObstacleMovingDistanceTolerance = 10.0f;

	/** The time an obstacle needs to be not moving before it is reported as stopped.*/
	UPROPERTY(EditAnywhere, config, Category = Obstacles, meta = (ClampMin = "0.0", UIMin = "0.0", ForceUnits = "s"))
	float ObstacleTimeToStop = 0.3f;

	/** The radius an obstacle has effects on navigation.*/
	UPROPERTY(EditAnywhere, config, Category = Obstacles, meta = (ClampMin = "0.0", UIMin = "0.0", ForceUnits = "cm"))
	float ObstacleEffectRadius = 1000.f;

protected:

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	/** Distance ahead of the current lane location for the next movement target location. */
	UPROPERTY(EditDefaultsOnly, config, meta = (ClampMin = "0.0", UIMin = "0.0"), Category = Movement)
	float MoveDistance = 500.f;

	/** Random deviation of the of the MoveDistance */
	UPROPERTY(EditDefaultsOnly, config, meta = (ClampMin = "0.0", UIMin = "0.0"), Category = Movement)
	float MoveDistanceRandomDeviation = 100.f;

	/** List of all lane density descriptors. */
	UPROPERTY(EditAnywhere, config, Category = "Lane Density")
	TArray<FMassCrowdLaneDensityDesc> LaneDensities;
};

#undef UE_API
