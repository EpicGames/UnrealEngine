// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassActorSpawnerSubsystem.h"
#include "MassCrowdSpawnerSubsystem.generated.h"

/**
 * A subsystem managing spawning of crowd
 */
UCLASS(MinimalAPI)
class UMassCrowdSpawnerSubsystem : public UMassActorSpawnerSubsystem
{
	GENERATED_BODY()
};
