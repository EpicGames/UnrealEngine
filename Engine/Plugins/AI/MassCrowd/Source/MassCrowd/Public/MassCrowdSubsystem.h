// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/EntityHandle.h"
#include "MassCrowdTypes.h"
#include "ZoneGraphTypes.h"
#include "MassSubsystemBase.h"
#include "MassCrowdSubsystem.generated.h"

class UZoneGraphAnnotationSubsystem;
class UMassCrowdSettings;
class UZoneGraphSubsystem;
class AZoneGraphData;

#if !UE_BUILD_SHIPPING
/** Called when a lane changes state at runtime. */
DECLARE_MULTICAST_DELEGATE(FDebugOnMassCrowdLaneStateChanged);
#endif

/** Structure holding all pertinent data related to the selected lane. */
struct FSelectLaneResult
{
	/** The handle of the next lane selected from the available links. */
	FZoneGraphLaneHandle NextLaneHandle;

	/** The distance along the lane the entity should reach before stopping. */
	TOptional<float> WaitDistance;
};

/** Container for the crowd lane data associated to a specific registered ZoneGraph data. */
struct FRegisteredCrowdLaneData
{
	void Reset()
	{
		CrowdLaneDataArray.Reset();
		DataHandle.Reset();
		LaneToTrackingDataLookup.Reset();
		LaneToBranchingDataLookup.Reset();
		WaitAreas.Reset();
	}

	/** Per lane data; array size matches the ZoneGraph storage. */
	TArray<FZoneGraphCrowdLaneData> CrowdLaneDataArray;

	/** Handle of the storage the data was initialized from. */
	FZoneGraphDataHandle DataHandle;

	/** Lane to entity tracking data lookup */
	TMap<int32, FCrowdTrackingLaneData> LaneToTrackingDataLookup;

	/** Lane to branching data lookup */
	TMap<int32, FCrowdBranchingLaneData> LaneToBranchingDataLookup;

	TArray<FCrowdWaitAreaData> WaitAreas;
};

/**
 * Subsystem that tracks mass entities that are wandering on the zone graph.
 * It will create custom runtime lane data to allow branching decisions.
 */
UCLASS(MinimalAPI)
class UMassCrowdSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
public:
#if !UE_BUILD_SHIPPING
	FDebugOnMassCrowdLaneStateChanged DebugOnMassCrowdLaneStateChanged;
#endif

#if WITH_EDITOR
	/** Clears and rebuilds all lane and intersection data for registered zone graphs using the current settings. */
	MASSCROWD_API void RebuildLaneData();
#endif

	/** @return true if the Crowd subsystem has lane data for specified graph. */
	MASSCROWD_API bool HasCrowdDataForZoneGraph(const FZoneGraphDataHandle DataHandle) const;

	/**
	 * Returns the readonly runtime data associated to a given zone graph.
	 * @param DataHandle A valid handle of the zone graph used to retrieve the runtime crowd data
	 * @return Runtime data associated to the zone graph if available; nullptr otherwise
	 * @note Method will ensure if DataHandle is invalid or if associated data doesn't exist. Should call HasCrowdDataForZoneGraph first.
	 */
	MASSCROWD_API const FRegisteredCrowdLaneData* GetCrowdData(const FZoneGraphDataHandle DataHandle) const;

	/**
	 * Returns the readonly runtime data associated to a given zone graph lane.
	 * @param LaneHandle A valid lane handle used to retrieve the runtime data; ensure if handle is invalid
	 * @return Runtime data associated to the lane (if available)
	 */
	MASSCROWD_API TOptional<FZoneGraphCrowdLaneData> GetCrowdLaneData(const FZoneGraphLaneHandle LaneHandle) const;

	/**
	 * Returns the entity tracking runtime data associated to a given zone graph lane.
	 * @param LaneHandle A valid lane handle used to retrieve the associated tracking data; ensure if handle is invalid
	 * @return Runtime data associated to the lane (nullptr if provided handle is invalid or no data is associated to that lane)
	 */
	MASSCROWD_API const FCrowdTrackingLaneData* GetCrowdTrackingLaneData(const FZoneGraphLaneHandle LaneHandle) const;

	/**
	 * Returns the branching data associated to a given zone graph lane.
 	 * @param LaneHandle A valid lane handle used to retrieve the associated data; ensure if handle is invalid
	 * @return Branching data associated to the lane (nullptr if provided handle is invalid or no data is associated to that lane)
	 */
	MASSCROWD_API const FCrowdBranchingLaneData* GetCrowdBranchingLaneData(const FZoneGraphLaneHandle LaneHandle) const;

	/**
	 * Returns the waiting area runtime data associated to a given zone graph lane.
	 * @param LaneHandle A valid lane handle used to retrieve the associated intersection data; ensure if handle is invalid
	 * @return Runtime data associated to the lane (nullptr if provided handle is invalid or no data is associated to that lane)
	 */
	MASSCROWD_API const FCrowdWaitAreaData* GetCrowdWaitingAreaData(const FZoneGraphLaneHandle LaneHandle) const;

	/**
	 * Return the current state of a lane.
	 * @param LaneHandle A valid lane handle used to retrieve the runtime data and change the lane state; ensure if handle is invalid
	 * @return The state of the lane
	 */
	MASSCROWD_API ECrowdLaneState GetLaneState(const FZoneGraphLaneHandle LaneHandle) const;

	/**
	 * Changes the state of a lane.
	 * @param LaneHandle A valid lane handle used to retrieve the runtime data and change the lane state; ensure if handle is invalid
	 * @param NewState The new state of the lane
	 * @return True if the state was successfully changed or was already in the right state; false if handle is invalid
	 */
	MASSCROWD_API bool SetLaneState(const FZoneGraphLaneHandle LaneHandle, ECrowdLaneState NewState);

	/** @return Combined Tag mask that represents all possible lane density tags. Built from MassCrowdSettings. */
	FZoneGraphTagMask GetDensityMask() const { return DensityMask; }

	/**
	 * Acquires a slot from a specified waiting lane. Nearest vacant slot to EntityPosition is returned. 
	 * @param Entity Requesting entity
	 * @param EntityPosition Position of the entity.
	 * @param LaneHandle A handle to a lane with waiting data.
	 * @param OutSlotPosition Position associated to the acquired slot.
	 * @param OutSlotDirection Facing direction associated to the acquire slot.
	 * @return Index of the slot, or INDEX_NONE if no slots are available or if the lane is not a waiting lane.
	 */
	MASSCROWD_API int32 AcquireWaitingSlot(const FMassEntityHandle Entity, const FVector& EntityPosition, const FZoneGraphLaneHandle LaneHandle,
							 FVector& OutSlotPosition, FVector& OutSlotDirection);
	
	/**
	 * Releases previously acquired slot from a specified waiting lane.
	 * @param Entity Requesting entity
	 * @param LaneHandle A handle to a lane with waiting data.
	 * @param SlotIndex Index of the previously acquired slot.
	 */
	MASSCROWD_API void ReleaseWaitingSlot(const FMassEntityHandle Entity, const FZoneGraphLaneHandle LaneHandle, const int32 SlotIndex);

	/**
	 * Callback from the lane tracker processor to indicates a mass entity changing lane.
	 * @param Entity The mass entity
	 * @param PreviousLaneHandle Last frame lane handle (can be invalid)
	 * @param CurrentLaneHandle Current frame lane handle (can be invalid)
	 */
	MASSCROWD_API void OnEntityLaneChanged(const FMassEntityHandle Entity, const FZoneGraphLaneHandle PreviousLaneHandle, const FZoneGraphLaneHandle CurrentLaneHandle);

	/** Returns the weight for lane selection that is associated to the given lane based on its density tag. */
	MASSCROWD_API float GetDensityWeight(const FZoneGraphLaneHandle LaneHandle, const FZoneGraphTagMask LaneTagMask) const;

protected:
	friend class UMassCrowdLaneTrackingProcessor;
	friend class UMassCrowdWanderFragmentDestructor;
	friend class UZoneGraphCrowdLaneAnnotations;

	MASSCROWD_API void UpdateDensityMask();

	// USubsystem BEGIN
	MASSCROWD_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSCROWD_API virtual void PostInitialize() override;
	MASSCROWD_API virtual void Deinitialize() override;
	// USubsystem END

	MASSCROWD_API void PostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData);
	MASSCROWD_API void PreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData);

	/**
	 * Returns the modifiable runtime data associated to a given zone graph lane.
	 * @param LaneHandle A valid lane handle used to retrieve the runtime data; ensure if handle is invalid
	 * @return Runtime data associated to the lane if available; nullptr otherwise
	 */
	MASSCROWD_API FZoneGraphCrowdLaneData* GetMutableCrowdLaneData(const FZoneGraphLaneHandle LaneHandle);

	/**
	 * Populates the crowd lane data array with all existing lane in the zone graph.
	 * Also creates some extra data for lanes that could be used to wait in case of
	 * temporarily closed lane (e.g. intersection crossings).
	 */
	MASSCROWD_API void BuildLaneData(FRegisteredCrowdLaneData& LaneData, const FZoneGraphStorage& Storage);

	/**
	 * Callback to keep count of entities currently on a given lane.
	 * @param Entity The entity entering the lane
	 * @param LaneIndex Index of the lane
	 * @param TrackingData Runtime state associated to the lane
	 */
	MASSCROWD_API void OnEnterTrackedLane(const FMassEntityHandle Entity, const int32 LaneIndex, FCrowdTrackingLaneData& TrackingData);

	/**
	 * Callback to keep count of entities currently on a given lane.
	 * @param Entity The entity exiting the lane
	 * @param LaneIndex Index of the lane
	 * @param TrackingData Runtime state associated to the lane
	 */
	MASSCROWD_API void OnExitTrackedLane(const FMassEntityHandle Entity, const int32 LaneIndex, FCrowdTrackingLaneData& TrackingData);

	/**
	 * Creates and initializes the occupancy data of a lane.
	 * @param LaneIndex Index of the source zone graph lane
	 * @param ZoneGraphStorage ZoneGraph data storage to extract lane information from
	 * @return The newly created structure holding occupancy data associated to the zone graph lane
	 */
	MASSCROWD_API FCrowdTrackingLaneData& CreateTrackingData(const int32 LaneIndex, const FZoneGraphStorage& ZoneGraphStorage);

	/**
	 * Creates dedicated structure to hold data of a branching lane.
	 * @param LaneIndex Index of the source zone graph lane
	 * @param ZoneGraphStorage ZoneGraph data storage to extract lane information from
	 * @return The newly created structure holding specific data of a branching lane
	 */
	MASSCROWD_API FCrowdBranchingLaneData& CreateBranchingData(const int32 LaneIndex, const FZoneGraphStorage& ZoneGraphStorage);

	MASSCROWD_API void CreateWaitSlots(const int32 CrossingLaneIndex, FCrowdWaitAreaData& WaitArea, const FZoneGraphStorage& ZoneGraphStorage);

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphSubsystem> ZoneGraphSubsystem = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphAnnotationSubsystem> ZoneGraphAnnotationSubsystem = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<const UMassCrowdSettings> MassCrowdSettings = nullptr;

	FDelegateHandle OnPostZoneGraphDataAddedHandle;
	FDelegateHandle OnPreZoneGraphDataRemovedHandle;
#if WITH_EDITOR
	FDelegateHandle OnMassCrowdSettingsChangedHandle;
	FDelegateHandle OnZoneGraphDataBuildDoneHandle;
#endif

	/** Per lane data for all registered ZoneGraph data. */
	TArray<FRegisteredCrowdLaneData> RegisteredLaneData;

	/** Tag mask that represents all possible lane density tags. Built from MassCrowdSettings. */
	FZoneGraphTagMask DensityMask;
};

template<>
struct TMassExternalSubsystemTraits<UMassCrowdSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};
