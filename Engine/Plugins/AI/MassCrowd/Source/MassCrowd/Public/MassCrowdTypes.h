// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassLODLogic.h"
#include "ZoneGraphTypes.h"
#include "MassCrowdTypes.generated.h"

struct FMassCrowdSimulationLODLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVariableTickRate = true,
	};
};

struct FMassCrowdViewerLODLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVisibilityLogic = true,
		bCalculateLODSignificance = true,
		bLocalViewersOnly = true,
	};
};

struct FMassCrowdCombinedLODLogic : public FLODDefaultLogic
{
	enum
	{
		bDoVariableTickRate = true,
		bDoVisibilityLogic = true,
		bCalculateLODSignificance = true,
		bLocalViewersOnly = true,
	};
};

/** State of a given lane */
UENUM()
enum class ECrowdLaneState : uint8
{
	Opened, // Pedestrians can enter the lane
	Closed, // Pedestrians can not enter the lane
};

/** Runtime data associated to lane that can be used to track count of entities on it. */
USTRUCT()
struct FCrowdTrackingLaneData
{
	GENERATED_BODY()

	/**
	 * Index of an associated waiting area. This will be valid for branching lanes leading
	 * to a crossing that requires agent to wait for incoming lane to open.
	 */
	int32 WaitAreaIndex = INDEX_NONE;

	int32 NumEntitiesOnLane = 0;
};

/** Runtime data associated to lane that can be used to wait another one to open. */
USTRUCT()
struct FCrowdWaitSlot
{
	GENERATED_BODY()

	FVector Position = FVector::Zero();
	
	FVector Forward = FVector::ForwardVector;

	float Radius = 0;

	bool bOccupied = false;
};

/** Runtime data associated to entry to a lane that can be opened or closed. */
USTRUCT()
struct FCrowdWaitAreaData
{
	GENERATED_BODY()

	void Reset()
	{
		Slots.Reset();
		NumFreeSlots = 0;
	}
	
	bool IsFull() const { return NumFreeSlots == 0; }
	int32 GetNumSlots() const { return Slots.Num(); }
	int32 GetNumFreeSlots() const { return NumFreeSlots; }
	int32 GetNumOccupiedSlots() const { return Slots.Num() - NumFreeSlots; }

	TArray<FCrowdWaitSlot> Slots;

	TArray<FZoneGraphLaneHandle> ConnectedLanes;
	
	int32 NumFreeSlots = 0;
};

/** Runtime data associated to branching. */
USTRUCT()
struct FCrowdBranchingLaneData
{
	GENERATED_BODY()

	/**
	 * Density mask extracted from the incoming lane.
	 * This is required since the intersection might split and reach different densities on connected shapes.
	 * We can't keep a single density for the whole polygon so we keep it per lane.
	 */
	uint32 DensityMask = 0;
};

/** Structure holding runtime data associated to a zone graph lane to handle pedestrian navigation. */
USTRUCT()
struct FZoneGraphCrowdLaneData
{
	GENERATED_BODY()
public:
	ECrowdLaneState GetState() const { return State; }
	void SetState(ECrowdLaneState Value) { State = Value; }
protected:
	ECrowdLaneState State = ECrowdLaneState::Opened;
};
