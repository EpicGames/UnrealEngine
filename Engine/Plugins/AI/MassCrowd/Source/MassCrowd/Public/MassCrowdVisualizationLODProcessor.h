// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassVisualizationLODProcessor.h"
#include "MassLODCollectorProcessor.h"
#include "MassCrowdVisualizationLODProcessor.generated.h"

#define UE_API MASSCROWD_API

/*
 * Created a crowd version for parallelization of the crowd with the traffic
 */
UCLASS(MinimalAPI, meta=(DisplayName="Crowd visualization LOD"))
class UMassCrowdVisualizationLODProcessor : public UMassVisualizationLODProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassCrowdVisualizationLODProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

/*
 * Created a crowd version for parallelization of the crowd with the traffic
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Crowd LOD Collection "))
class UMassCrowdLODCollectorProcessor : public UMassLODCollectorProcessor
{
	GENERATED_BODY()

	UE_API UMassCrowdLODCollectorProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

#undef UE_API
