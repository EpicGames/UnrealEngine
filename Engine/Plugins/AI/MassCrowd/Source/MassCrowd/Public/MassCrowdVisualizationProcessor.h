// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationProcessor.h"
#include "MassCrowdVisualizationProcessor.generated.h"

#define UE_API MASSCROWD_API

/**
 * Overridden visualization processor to make it tied to the crowd via the requirements
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Mass Crowd Visualization"))
class UMassCrowdVisualizationProcessor : public UMassVisualizationProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassCrowdVisualizationProcessor();

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

/**
 * A custom visualization processor for debugging mass crowd
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Mass Crowd Visualization"))
class UMassDebugCrowdVisualizationProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassDebugCrowdVisualizationProcessor();

protected:

	/** Configure the owned FMassEntityQuery instances to express processor's requirements */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;

	/**
	 * Execution method for this processor
	 * @param EntitySubsystem is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	UPROPERTY(Transient)
	TObjectPtr<UWorld> World;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
