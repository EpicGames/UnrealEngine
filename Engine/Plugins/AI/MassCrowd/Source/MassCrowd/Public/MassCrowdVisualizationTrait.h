// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassVisualizationTrait.h"
#include "MassCrowdVisualizationTrait.generated.h"

#define UE_API MASSCROWD_API

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta=(DisplayName="Crowd Visualization"))
class UMassCrowdVisualizationTrait : public UMassVisualizationTrait
{
	GENERATED_BODY()
public:
	UE_API UMassCrowdVisualizationTrait();

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
