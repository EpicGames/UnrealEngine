// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "ZoneGraphTypes.h"
#include "Tasks/MassZoneGraphPathFollowTask.h"
#include "MassCrowdClaimWaitSlotTask.generated.h"

#define UE_API MASSCROWD_API

struct FStateTreeExecutionContext;
class UMassCrowdSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
* Claim wait slot and expose slot position for path follow.
*/
USTRUCT()
struct FMassCrowdClaimWaitSlotTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Output)
	FMassZoneGraphTargetLocation WaitSlotLocation;

	UPROPERTY()
	int32 WaitingSlotIndex = INDEX_NONE;
	
	UPROPERTY()
	FZoneGraphLaneHandle AcquiredLane;
};

USTRUCT(meta = (DisplayName = "Crowd Claim Wait Slot"))
struct FMassCrowdClaimWaitSlotTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassCrowdClaimWaitSlotTaskInstanceData;
	
	UE_API FMassCrowdClaimWaitSlotTask();

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<UMassCrowdSubsystem> CrowdSubsystemHandle;
};

#undef UE_API
