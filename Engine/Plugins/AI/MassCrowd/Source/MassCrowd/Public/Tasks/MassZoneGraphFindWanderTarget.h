// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "Tasks/MassZoneGraphPathFollowTask.h"
#include "MassZoneGraphFindWanderTarget.generated.h"

#define UE_API MASSCROWD_API

struct FStateTreeExecutionContext;
struct FMassZoneGraphLaneLocationFragment;
class UZoneGraphSubsystem;
class UZoneGraphAnnotationSubsystem;
class UMassCrowdSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Updates TargetLocation to a wander target based on the agents current location on ZoneGraph.
 */
USTRUCT()
struct FMassZoneGraphFindWanderTargetInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Output)
	FMassZoneGraphTargetLocation WanderTargetLocation;
};

USTRUCT(meta = (DisplayName = "ZG Find Wander Target"))
struct FMassZoneGraphFindWanderTarget : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphFindWanderTargetInstanceData;
	
	UE_API FMassZoneGraphFindWanderTarget();

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<UZoneGraphSubsystem> ZoneGraphSubsystemHandle;
	TStateTreeExternalDataHandle<UZoneGraphAnnotationSubsystem> ZoneGraphAnnotationSubsystemHandle;
	TStateTreeExternalDataHandle<UMassCrowdSubsystem> MassCrowdSubsystemHandle;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FZoneGraphTagFilter AllowedAnnotationTags;
};

#undef UE_API
