// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCrowdSubsystem.h"
#include "ZoneGraphAnnotationComponent.h"
#include "ZoneGraphAnnotationTypes.h"

#include "ZoneGraphCrowdLaneAnnotations.generated.h"

#define UE_API MASSCROWD_API

/** Event indicating the new state of a lane. */
USTRUCT()
struct FZoneGraphCrowdLaneStateChangeEvent : public FZoneGraphAnnotationEventBase
{
	GENERATED_BODY()

	FZoneGraphCrowdLaneStateChangeEvent() = default;
	FZoneGraphCrowdLaneStateChangeEvent(const FZoneGraphLaneHandle LaneHandle, const ECrowdLaneState NewState)
		: Lane(LaneHandle), State(NewState)	{}

	/** Affected lane. */
	UPROPERTY()
	FZoneGraphLaneHandle Lane;

	/** New state. */
	UPROPERTY()
	ECrowdLaneState State = ECrowdLaneState::Opened;
};

/**
 * Zone graph blocking behavior
 */
UCLASS(MinimalAPI, ClassGroup = AI, BlueprintType, meta = (BlueprintSpawnableComponent))
class UZoneGraphCrowdLaneAnnotations : public UZoneGraphAnnotationComponent
{
	GENERATED_BODY()

protected:
	UE_API virtual void PostSubsystemsInitialized() override;
	UE_API virtual FZoneGraphTagMask GetAnnotationTags() const override;
	UE_API virtual void HandleEvents(const FInstancedStructContainer& Events) override;
	UE_API virtual void TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& AnnotationTagContainer) override;

#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual void DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy) override;
	UE_API virtual void DebugDrawCanvas(UCanvas* Canvas, APlayerController*) override;
#endif // UE_ENABLE_DEBUG_DRAWING

	/** Annotation Tag to mark a closed lane. */
	UPROPERTY(EditAnywhere, Category = CrowdLane)
	FZoneGraphTag CloseLaneTag;

	/** Annotation Tag to mark a waiting lane. */
	UPROPERTY(EditAnywhere, Category = CrowdLane)
	FZoneGraphTag WaitingLaneTag;

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDisplayTags = false;

	/** Array of queued events. */
	TArray<FZoneGraphCrowdLaneStateChangeEvent> StateChangeEvents;

	/** Cached ZoneGraphSubsystem */
	UPROPERTY(Transient)
	TObjectPtr<UMassCrowdSubsystem> CrowdSubsystem = nullptr;
};

#undef UE_API
